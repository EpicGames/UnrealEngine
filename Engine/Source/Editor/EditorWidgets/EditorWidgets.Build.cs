// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class EditorWidgets : ModuleRules
{
	public EditorWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PrivateIncludePathModuleNames.Add("AssetRegistry");

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"CoreUObject",
				"ToolWidgets",
				"EditorConfig",
				"AssetDefinition",
			}
		);
		
		DynamicallyLoadedModuleNames.Add("AssetRegistry");
	}
}
