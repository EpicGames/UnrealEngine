// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailNameAreaObjectFilterRegistry.h"

#define LOCTEXT_NAMESPACE "EditorWidgets"

namespace UE::EditorWidgets
{

void FDetailNameAreaObjectFilterRegistry::RegisterDetailNameAreaObjectFilter(TUniquePtr<IDetailNameAreaObjectFilter>&& NewNameAreaFilter)
{
	const FName FilterName = NewNameAreaFilter->GetFilterName();
	ObjectFilters.Add({ FilterName, MoveTemp(NewNameAreaFilter) });
}

void FDetailNameAreaObjectFilterRegistry::UnregisterDetailNameAreaObjectFilter(const FName& FilterName)
{
	ObjectFilters.Remove(FilterName);
}

void FDetailNameAreaObjectFilterRegistry::ApplyAllFiltersToSelectedObjects(TArray<TWeakObjectPtr<UObject>>& InOutObjects) const
{
	for (const TPair<FName, TUniquePtr<IDetailNameAreaObjectFilter>>& Filter : ObjectFilters)
	{
		Filter.Value->FilterSelectedObjectsForNameArea(InOutObjects);
	}
}

} // end namespace UE::EditorWidgets

#undef LOCTEXT_NAMESPACE
