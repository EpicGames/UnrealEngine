// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorWidgetsModule.h"

#include "EditorWidgetsStyle.h"
#include "HAL/Platform.h"
#include "Modules/ModuleManager.h"
#include "ObjectNameEditSinkRegistry.h"
#include "DetailNameAreaObjectFilterRegistry.h"
#include "SAssetDiscoveryIndicator.h"
#include "SObjectNameEditableTextBox.h"
#include "STransportControl.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SWidget;
class UObject;

IMPLEMENT_MODULE( FEditorWidgetsModule, EditorWidgets );

const FName FEditorWidgetsModule::EditorWidgetsAppIdentifier( TEXT( "EditorWidgetsApp" ) );

void FEditorWidgetsModule::StartupModule()
{
	FEditorWidgetsStyle::Initialize();
	
	ObjectNameEditSinkRegistry = MakeShared<UE::EditorWidgets::FObjectNameEditSinkRegistry>();
	DetailNameAreaObjectFilterRegistry = MakeShared<UE::EditorWidgets::FDetailNameAreaObjectFilterRegistry>();
}

void FEditorWidgetsModule::ShutdownModule()
{
	ObjectNameEditSinkRegistry.Reset();
	DetailNameAreaObjectFilterRegistry.Reset();

	FEditorWidgetsStyle::Shutdown();
}

TSharedRef<IObjectNameEditableTextBox> FEditorWidgetsModule::CreateObjectNameEditableTextBox(const TArray<TWeakObjectPtr<UObject>>& Objects)
{
	TSharedRef<SObjectNameEditableTextBox> Widget = SNew(SObjectNameEditableTextBox).Objects(Objects).Registry(ObjectNameEditSinkRegistry);
	return Widget;
}

TSharedRef<SWidget> FEditorWidgetsModule::CreateAssetDiscoveryIndicator(EAssetDiscoveryIndicatorScaleMode::Type ScaleMode, FMargin Padding, bool bFadeIn)
{
	return SNew(SAssetDiscoveryIndicator)
		.ScaleMode(ScaleMode)
		.Padding(Padding)
		.FadeIn(bFadeIn);
}

TSharedRef<ITransportControl> FEditorWidgetsModule::CreateTransportControl(const FTransportControlArgs& Args)
{
	return SNew(STransportControl)
		.TransportArgs(Args);
}

TSharedRef<UE::EditorWidgets::FObjectNameEditSinkRegistry> FEditorWidgetsModule::GetObjectNameEditSinkRegistry() const
{
	return ObjectNameEditSinkRegistry.ToSharedRef();
}

TSharedRef<UE::EditorWidgets::FDetailNameAreaObjectFilterRegistry> FEditorWidgetsModule::GetDetailNameAreaObjectFilterRegistry() const
{
	return DetailNameAreaObjectFilterRegistry.ToSharedRef();
}
