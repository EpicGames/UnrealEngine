// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorWidgetsStyle.h"

#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "Styling/SlateStyleMacros.h"

class ISlateStyle;

TSharedPtr<FSlateStyleSet> FEditorWidgetsStyle::StyleSet = nullptr;

#define RootToContentDir StyleSet->RootToContentDir

void FEditorWidgetsStyle::Initialize()
{
	// Only register once
	if(StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet("EditorWidgets"));

	const FEditableTextBoxStyle& NormalEditableTextBoxStyle = FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");

	const FTextBlockStyle NormalText = FTextBlockStyle()
		.SetColorAndOpacity(NormalEditableTextBoxStyle.ForegroundColor)
		.SetHighlightColor(NormalEditableTextBoxStyle.FocusedForegroundColor)
		.SetFont(NormalEditableTextBoxStyle.TextStyle.Font)
		.SetFontSize(NormalEditableTextBoxStyle.TextStyle.Font.Size);

	const FEditableTextBoxStyle TextBoxStyle = FEditableTextBoxStyle(NormalEditableTextBoxStyle).SetTextStyle(NormalText);


	// Text editor
	{
		StyleSet->Set("NormalEditableTextBox", TextBoxStyle);
		StyleSet->Set("TextEditor.NormalText", NormalText);

		StyleSet->Set("SyntaxHighlight.Template.Normal", NormalText);
		StyleSet->Set("SyntaxHighlight.Template.Argument", FAppStyle::GetWidgetStyle<FTextBlockStyle>("RichTextBlock.BoldHighlight"));
	}

	FSlateStyleRegistry::RegisterSlateStyle( *StyleSet.Get() );
}

#undef RootToContentDir

void FEditorWidgetsStyle::Shutdown()
{
	if(StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

const ISlateStyle& FEditorWidgetsStyle::Get()
{
	return *(StyleSet.Get());
}

const FName& FEditorWidgetsStyle::GetStyleSetName()
{
	return StyleSet->GetStyleSetName();
}
