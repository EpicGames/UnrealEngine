// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/CustomClassFilterData.h"

#include "AssetRegistry/ARFilter.h"
#include "IAssetTypeActions.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "Misc/AssetFilterData.h"

FCustomClassFilterData::FCustomClassFilterData(UAssetDefinition* InAssetDefinition, const FAssetFilterData& InAssetTypeAction)
	: AssetDefinitionPtr(InAssetDefinition)
{
	Class = InAssetDefinition->GetAssetClass().Get();
	FilterName = InAssetTypeAction.Name;
	FilterDisplayName = InAssetTypeAction.DisplayText;
	ClassPathName = InAssetDefinition->GetAssetClass().ToSoftObjectPath().GetAssetPath();

	Filter = InAssetTypeAction.Filter;
}

FCustomClassFilterData::FCustomClassFilterData(UClass* InClass, TSharedPtr<FFilterCategory> InCategory, FLinearColor InColor)
	: Class(InClass)
	, Color(InColor)
{
	Categories.Add(InCategory);
	ClassPathName = Class->GetClassPathName();
	FilterName = Class->GetFName().ToString();
	FilterDisplayName = Class->GetDisplayNameText();

	Filter.ClassPaths.Add(Class->GetClassPathName());
	Filter.bRecursiveClasses = true;
}

void FCustomClassFilterData::AddCategory(TSharedPtr<FFilterCategory> InCategory)
{
	Categories.AddUnique(InCategory);
}

UClass* FCustomClassFilterData::GetClass() const
{
	return Class.Get();
}

TArray<TSharedPtr<FFilterCategory>> FCustomClassFilterData::GetCategories() const
{
	return Categories;
}

FLinearColor FCustomClassFilterData::GetColor() const
{
	// Kept this one because some assets can have their color rebound, is it worth keeping it dynamic here?  meh.
	if (const UAssetDefinition* AssetDefinition = AssetDefinitionPtr.Get())
	{
		return AssetDefinition->GetAssetColor();
	}

	return Color;
}

void FCustomClassFilterData::BuildBackendFilter(FARFilter& OutFilter)
{
	OutFilter = Filter;
}

FText FCustomClassFilterData::GetName() const
{
	return FilterDisplayName;
}

FString FCustomClassFilterData::GetFilterName() const
{
	return FilterName;
}

FTopLevelAssetPath FCustomClassFilterData::GetClassPathName() const
{
	return ClassPathName;
}
