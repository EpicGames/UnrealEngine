// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/FilterBarConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FilterBarConfig)

TObjectPtr<UFilterBarConfig> UFilterBarConfig::Instance = nullptr;

void UFilterBarConfig::Initialize()
{
	if(!Instance)
	{
		Instance = NewObject<UFilterBarConfig>(); 
		Instance->AddToRoot();
	}
}
