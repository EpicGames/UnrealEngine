// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/SAssetFilterBar.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SAssetFilterBar)

#define LOCTEXT_NAMESPACE "FilterBar"

namespace UE::Editor::Widgets
{
	FFilterCategoryMenu::FFilterCategoryMenu(
		const FName& InSectionExtensionHook,
		const FText& InSectionHeading)
		: SectionExtensionHook(InSectionExtensionHook)
		, SectionHeading(InSectionHeading)
	{}

	TMap<TSharedPtr<FFilterCategory>, FFilterCategoryMenu> BuildCategoryToMenuMap(
		TMap<FName, TSharedPtr<FFilterCategory>>& InAssetFilterCategories,
		TConstArrayView<TSharedRef<FCustomClassFilterData>> InCustomClassFilters, const FOnFilterAssetType& InOnFilterAssetType)
	{
		// Create a map of Categories to Menus
		TMap<TSharedPtr<FFilterCategory>, FFilterCategoryMenu> CategoryToMenuMap;

		// For every asset type, move it into all the categories it should appear in
		for (const TSharedRef<FCustomClassFilterData>& CustomClassFilter : InCustomClassFilters)
		{
			bool bPassesExternalFilters = true;

			// Run any external class filters we have
			if (InOnFilterAssetType.IsBound())
			{
				bPassesExternalFilters = InOnFilterAssetType.Execute(CustomClassFilter->GetClass());
			}

			if (bPassesExternalFilters)
			{
				/** Get all the categories this filter belongs to */
				TArray<TSharedPtr<FFilterCategory>> Categories = CustomClassFilter->GetCategories();

				for (const TSharedPtr<FFilterCategory>& Category : Categories)
				{
					// If the category for this custom class already exists
					if (FFilterCategoryMenu* CategoryMenu = CategoryToMenuMap.Find(Category))
					{
						CategoryMenu->Classes.Add( CustomClassFilter );
					}
					// Otherwise create a new FCategoryMenu for the category and add it to the map
					else
					{
						const FText SectionHeading = FText::Format(LOCTEXT("WildcardFilterHeadingHeadingTooltip", "{0} Filters"), Category->Title);
						const FName ExtensionPoint = FName(FText::AsCultureInvariant(SectionHeading).ToString());

						FFilterCategoryMenu NewCategoryMenu(ExtensionPoint, SectionHeading);
						NewCategoryMenu.Classes.Add(CustomClassFilter);

						CategoryToMenuMap.Add(Category, NewCategoryMenu);
					}
				}
			}
		}

		// Remove any empty categories
		for (auto MenuIt = CategoryToMenuMap.CreateIterator(); MenuIt; ++MenuIt)
		{
			if (MenuIt.Value().Classes.Num() == 0)
			{
				CategoryToMenuMap.Remove(MenuIt.Key());
			}
		}

		// Set the extension hook for the basic category, if it exists and we have any assets for it
		if (TSharedPtr<FFilterCategory>* BasicCategory = InAssetFilterCategories.Find(EAssetCategoryPaths::Basic.GetCategory()))
		{
			if (FFilterCategoryMenu* BasicMenu = CategoryToMenuMap.Find(*BasicCategory))
			{
				BasicMenu->SectionExtensionHook = "FilterBarFilterBasicAsset";
			}
		}

		return CategoryToMenuMap;
	}
}

FFilterBarSettings* FFilterBarBase::GetMutableConfig()
{
	if(FilterBarIdentifier.IsNone())
	{
		return nullptr;
	}

	UFilterBarConfig* FilterBarConfig = GetMutableDefault<UFilterBarConfig>();
	return &UFilterBarConfig::Get()->FilterBars.FindOrAdd(FilterBarIdentifier);
}

const FFilterBarSettings* FFilterBarBase::GetConstConfig() const
{
	if(FilterBarIdentifier.IsNone())
	{
		return nullptr;
	}

	const UFilterBarConfig* FilterBarConfig = GetDefault<UFilterBarConfig>();
	return UFilterBarConfig::Get()->FilterBars.Find(FilterBarIdentifier);
}

void FFilterBarBase::SaveConfig()
{
	UFilterBarConfig::Get()->SaveEditorConfig();
}

void FFilterBarBase::InitializeConfig()
{
	UFilterBarConfig::Initialize();

	UFilterBarConfig::Get()->LoadEditorConfig();

	// Call GetMutableConfig to force create a config for this filter bar if the user specified FilterBarIdentifier
	FFilterBarSettings* FilterBarConfig = GetMutableConfig();
}

#undef LOCTEXT_NAMESPACE
