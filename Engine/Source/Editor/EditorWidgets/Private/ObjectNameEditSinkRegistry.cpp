// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectNameEditSinkRegistry.h"

#include "Containers/UnrealString.h"
#include "Editor/EditorEngine.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "IObjectNameEditSink.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/Object.h"

#define LOCTEXT_NAMESPACE "EditorWidgets"

namespace UE::EditorWidgets
{

/** Name edit sink implementation for a generic UObject. Used for anything without a specific implementation. */
class FObjectNameEditSink : public IObjectNameEditSink
{
	virtual UClass* GetSupportedClass() const override
	{
		return UObject::StaticClass();
	}

	virtual FText GetObjectDisplayName(UObject* Object) const override
	{
		return FText::FromString(Object->GetName());
	}

	virtual FText GetObjectNameTooltip(UObject* Object) const override
	{
		return LOCTEXT("EditableActorLabel_NoEditObjectTooltip", "Can't rename selected object (only actors can have editable labels)");
	}
};

class FActorNameEditSink : public IObjectNameEditSink
{
	virtual UClass* GetSupportedClass() const override
	{
		return AActor::StaticClass();
	}

	virtual FText GetObjectDisplayName(UObject* Object) const override
	{
		return FText::FromString(CastChecked<AActor>(Object)->GetActorLabel());
	}

	virtual bool IsObjectDisplayNameReadOnly(UObject* Object) const override
	{
		return !(CastChecked<AActor>(Object)->IsActorLabelEditable());
	};

	virtual bool SetObjectDisplayName(UObject* Object, FString DisplayName) override
	{
		if (IsObjectDisplayNameReadOnly(Object))
		{
			return false;
		}

		AActor* Actor = CastChecked<AActor>(Object);

		if (Actor->GetActorLabel() == DisplayName)
		{
			return false;
		}

		FActorLabelUtilities::RenameExistingActor(Actor, DisplayName);
		return true;
	}

	virtual FText GetObjectNameTooltip(UObject* Object) const override
	{
		if (IsObjectDisplayNameReadOnly(Object))
		{
			return LOCTEXT("EditableActorLabel_NoEditActorTooltip", "Can't rename selected actor (its label isn't editable)");
		}

		return FText::Format(LOCTEXT("EditableActorLabel_ActorTooltipFmt", "Rename the selected {0}"), FText::FromString(Object->GetClass()->GetName()));
	}
};

FObjectNameEditSinkRegistry::FObjectNameEditSinkRegistry()
{
	// Register default name edit sinks
	RegisterObjectNameEditSink(MakeShared<FObjectNameEditSink>());
	RegisterObjectNameEditSink(MakeShared<FActorNameEditSink>());
}

void FObjectNameEditSinkRegistry::RegisterObjectNameEditSink(const TSharedRef<IObjectNameEditSink>& NewSink)
{
	ObjectNameEditSinkList.Add(NewSink);
}

void FObjectNameEditSinkRegistry::UnregisterObjectNameEditSink(const TSharedRef<IObjectNameEditSink>& SinkToRemove)
{
	ObjectNameEditSinkList.Remove(SinkToRemove);
}

TSharedPtr<IObjectNameEditSink> FObjectNameEditSinkRegistry::GetObjectNameEditSinkForClass(const UClass* Class) const
{
	TSharedPtr<IObjectNameEditSink> MostDerivedObjectNameEditSink;

	for (const TSharedRef<IObjectNameEditSink>& NameEditSink : ObjectNameEditSinkList)
	{
		UClass* SupportedClass = NameEditSink->GetSupportedClass();

		if (Class->IsChildOf(SupportedClass))
		{
			if (!MostDerivedObjectNameEditSink.IsValid() ||
				SupportedClass->IsChildOf(MostDerivedObjectNameEditSink->GetSupportedClass()))
			{
				MostDerivedObjectNameEditSink = NameEditSink;
			}
		}
	}

	return MostDerivedObjectNameEditSink;
}

} // end namespace UE::EditorWidgets

#undef LOCTEXT_NAMESPACE
