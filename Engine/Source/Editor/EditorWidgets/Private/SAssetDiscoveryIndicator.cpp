// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssetDiscoveryIndicator.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Text/TextLayout.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AssetDiscoveryIndicator"

namespace AssetDiscoveryIndicatorConstants
{
	static const FMargin Padding(12, 4);
	static const FMargin SubStatusTextPadding(6, 2, 6, 0);
}

SAssetDiscoveryIndicator::~SAssetDiscoveryIndicator()
{
	if ( FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")) )
	{
		// The engine might be shutting down so we need to use TryGet and handle null
		IAssetRegistry* AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).TryGet();
		if (AssetRegistry) 
		{
			AssetRegistry->OnFileLoadProgressUpdated().RemoveAll(this);
			AssetRegistry->OnFilesLoaded().RemoveAll(this);
		}
	}
}

void SAssetDiscoveryIndicator::Construct( const FArguments& InArgs )
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnFileLoadProgressUpdated().AddSP( this, &SAssetDiscoveryIndicator::OnAssetRegistryFileLoadProgress );
	AssetRegistryModule.Get().OnFilesLoaded().AddSP( this, &SAssetDiscoveryIndicator::OnAssetRegistryFilesLoaded );

	ScaleMode = InArgs._ScaleMode;

	FadeAnimation = FCurveSequence();
	FadeAnimation.AddCurve(0.f, 4.f); // Add some space at the beginning to delay before fading in
	ScaleCurve = FadeAnimation.AddCurve(4.f, 0.75f);
	FadeCurve = FadeAnimation.AddCurve(4.75f, 0.75f);
	FadeAnimation.AddCurve(5.5f, 1.f); // Add some space at the end to cause a short delay before fading out

	MainStatusText = LOCTEXT("InitializingAssetDiscovery", "Initializing Asset Discovery");
	StatusTextWrapWidth = 0.0f;

	if ( AssetRegistryModule.Get().IsLoadingAssets() )
	{
		// Loading assets, marquee while discovering package files
		Progress = TOptional<float>();

		if ( InArgs._FadeIn )
		{
			FadeAnimation.Play( this->AsShared() );
		}
		else
		{
			FadeAnimation.JumpToEnd();
		}
	}
	else
	{
		// Already done loading assets, set to complete and don't play the complete animation
		Progress = 1.f;
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(InArgs._Padding)
		.HAlign(HAlign_Center)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			.BorderBackgroundColor(this, &SAssetDiscoveryIndicator::GetBorderBackgroundColor)
			.ColorAndOpacity(this, &SAssetDiscoveryIndicator::GetIndicatorColorAndOpacity)
			.DesiredSizeScale(this, &SAssetDiscoveryIndicator::GetIndicatorDesiredSizeScale)
			.Visibility(this, &SAssetDiscoveryIndicator::GetIndicatorVisibility)
			.VAlign(VAlign_Center)
			[
				SNew(SVerticalBox)

				// Text
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(AssetDiscoveryIndicatorConstants::Padding)
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("AssetDiscoveryIndicator.MainStatusFont"))
						.Text(this, &SAssetDiscoveryIndicator::GetMainStatusText)
						.WrapTextAt(this, &SAssetDiscoveryIndicator::GetStatusTextWrapWidth)
						.Justification(ETextJustify::Center)
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBox)
						.Padding(AssetDiscoveryIndicatorConstants::SubStatusTextPadding)
						.Visibility(this, &SAssetDiscoveryIndicator::GetSubStatusTextVisibility)
						[
							SNew(STextBlock)
							.Font(FAppStyle::GetFontStyle("AssetDiscoveryIndicator.SubStatusFont"))
							.Text(this, &SAssetDiscoveryIndicator::GetSubStatusText)
							.WrapTextAt(this, &SAssetDiscoveryIndicator::GetStatusTextWrapWidth)
							.Justification(ETextJustify::Center)
						]
					]
				]

				// Progress bar
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(AssetDiscoveryIndicatorConstants::Padding)
				[
					SNew(SProgressBar)
					.Percent(this, &SAssetDiscoveryIndicator::GetProgress)
				]
			]
		]
	];
}

void SAssetDiscoveryIndicator::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// Calculate the wrapping width based on our complete alloted width
	// We do this rather than auto-wrap because the size of the text changes, and auto-wrapping prevents the text block from being able to grow if the text shrinks
	StatusTextWrapWidth = AllottedGeometry.GetLocalSize().X - (AssetDiscoveryIndicatorConstants::Padding.Left + AssetDiscoveryIndicatorConstants::Padding.Right);
}

void SAssetDiscoveryIndicator::OnAssetRegistryFileLoadProgress(const IAssetRegistry::FFileLoadProgressUpdateData& ProgressUpdateData)
{
	if (ProgressUpdateData.bIsDiscoveringAssetFiles)
	{
		// Marquee while we're discovering asset files as we can't yet show an accurate percentage
		Progress.Reset();

		// Once we have discovered enough assets, show a preview of the progress bar so it doesn't jump from 0 to 50%
		if (ProgressUpdateData.NumAssetsProcessedByAssetRegistry > 10000 && ProgressUpdateData.NumTotalAssets > 0)
		{
			Progress = static_cast<float>(ProgressUpdateData.NumAssetsProcessedByAssetRegistry) / static_cast<float>(ProgressUpdateData.NumTotalAssets);
		}

		MainStatusText = LOCTEXT("DiscoveringAssetFiles", "Discovering Asset Files");
		SubStatusText = FText::Format(LOCTEXT("XFilesFoundFmt", "{0} files found"), FText::AsNumber(ProgressUpdateData.NumTotalAssets));
	}
	else
	{
		if (ProgressUpdateData.NumTotalAssets > 0)
		{
			Progress = static_cast<float>(ProgressUpdateData.NumAssetsProcessedByAssetRegistry) / static_cast<float>(ProgressUpdateData.NumTotalAssets);
		}
		else
		{
			Progress = 0.0f;
		}

		if (ProgressUpdateData.NumAssetsPendingDataLoad > 0)
		{
			MainStatusText = LOCTEXT("DiscoveringAssetData", "Discovering Asset Data");
			SubStatusText = FText::Format(LOCTEXT("XAssetsRemainingFmt", "{0} assets remaining"), FText::AsNumber(ProgressUpdateData.NumAssetsPendingDataLoad));
		}
		else
		{
			const int32 NumAssetsLeftToProcess = ProgressUpdateData.NumTotalAssets - ProgressUpdateData.NumAssetsProcessedByAssetRegistry;
			if (NumAssetsLeftToProcess == 0)
			{
				OnAssetRegistryFilesLoaded();
			}
			else
			{
				MainStatusText = LOCTEXT("ProcessingAssetData", "Processing Asset Data");
				SubStatusText = FText::Format(LOCTEXT("XAssetsRemainingFmt", "{0} assets remaining"), FText::AsNumber(NumAssetsLeftToProcess));
			}
		}
	}
}

void SAssetDiscoveryIndicator::OnAssetRegistryFilesLoaded()
{
	if (!FadeAnimation.IsAtStart())
	{
		MainStatusText = LOCTEXT("FinishedAssetDiscovery", "Finished Asset Discovery");
		SubStatusText = FText::GetEmpty();

		if (FadeAnimation.IsPlaying())
		{
			if (FadeAnimation.IsForward())
			{
				// Still fading in - reverse so we fade back out
				FadeAnimation.Reverse();
			}
			else
			{
				// Do nothing - already fading out
			}
		}
		else
		{
			// Play the fade out animation
			FadeAnimation.PlayReverse(this->AsShared());
		}
	}
}

FText SAssetDiscoveryIndicator::GetMainStatusText() const
{
	return MainStatusText;
}

FText SAssetDiscoveryIndicator::GetSubStatusText() const
{
	return SubStatusText;
}

TOptional<float> SAssetDiscoveryIndicator::GetProgress() const
{
	return Progress;
}

EVisibility SAssetDiscoveryIndicator::GetSubStatusTextVisibility() const
{
	return (SubStatusText.IsEmpty()) ? EVisibility::Collapsed : EVisibility::Visible;
}

float SAssetDiscoveryIndicator::GetStatusTextWrapWidth() const
{
	return StatusTextWrapWidth;
}

FSlateColor SAssetDiscoveryIndicator::GetBorderBackgroundColor() const
{
	return FSlateColor(FLinearColor(1, 1, 1, 0.8f * FadeCurve.GetLerp()));
}

FLinearColor SAssetDiscoveryIndicator::GetIndicatorColorAndOpacity() const
{
	return FLinearColor(1, 1, 1, FadeCurve.GetLerp());
}

FVector2D SAssetDiscoveryIndicator::GetIndicatorDesiredSizeScale() const
{
	const float Lerp = ScaleCurve.GetLerp();

	switch (ScaleMode)
	{
	case EAssetDiscoveryIndicatorScaleMode::Scale_Horizontal: return FVector2D(Lerp, 1);
	case EAssetDiscoveryIndicatorScaleMode::Scale_Vertical: return FVector2D(1, Lerp);
	case EAssetDiscoveryIndicatorScaleMode::Scale_Both: return FVector2D(Lerp, Lerp);
	default:
		return FVector2D(1, 1);
	}
}

EVisibility SAssetDiscoveryIndicator::GetIndicatorVisibility() const
{
	return FadeAnimation.IsAtStart() ? EVisibility::Collapsed : EVisibility::HitTestInvisible;
}

#undef LOCTEXT_NAMESPACE
