// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/CurveHandle.h"
#include "Animation/CurveSequence.h"
#include "AssetDiscoveryIndicator.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

struct FGeometry;

template <>
struct TWidgetTypeTraits<class SAssetDiscoveryIndicator>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** An indicator for the progress of the asset registry background search */
class SAssetDiscoveryIndicator : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SAssetDiscoveryIndicator )
		: _ScaleMode(EAssetDiscoveryIndicatorScaleMode::Scale_None)
		, _FadeIn(true)
		{
			// This widget has no interactive controls and is used as an overlay,
			// so removing it from the hit test grid will stop it from eating scroll events.
			_Visibility = EVisibility::SelfHitTestInvisible;
		}

		/** The way the indicator will scale out when done displaying progress */
		SLATE_ARGUMENT( EAssetDiscoveryIndicatorScaleMode::Type, ScaleMode )

		/** The padding to apply to the background of the indicator */
		SLATE_ARGUMENT( FMargin, Padding )

		/** If true, this widget will fade in after a short delay */
		SLATE_ARGUMENT( bool, FadeIn )

	SLATE_END_ARGS()

	/** Destructor */
	virtual ~SAssetDiscoveryIndicator();

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	// SCompoundWidget interface
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	/** Handles updating the progress from the asset registry */
	void OnAssetRegistryFileLoadProgress(const IAssetRegistry::FFileLoadProgressUpdateData& ProgressUpdateData);

	/** Handles updating the progress from the asset registry */
	void OnAssetRegistryFilesLoaded();

	/** Gets the main status text */
	FText GetMainStatusText() const;

	/** Gets the sub status text */
	FText GetSubStatusText() const;

	/** Gets the progress bar fraction */
	TOptional<float> GetProgress() const;

	/** Gets the sub status text visbility */
	EVisibility GetSubStatusTextVisibility() const;

	/** Get the current wrap point for the status text */
	float GetStatusTextWrapWidth() const;

	/** Gets the background's opacity */
	FSlateColor GetBorderBackgroundColor() const;

	/** Gets the whole widget's opacity */
	FLinearColor GetIndicatorColorAndOpacity() const;

	/** Gets the whole widget's opacity */
	FVector2D GetIndicatorDesiredSizeScale() const;

	/** Gets the whole widget's visibility */
	EVisibility GetIndicatorVisibility() const;

private:
	/** The main status text */
	FText MainStatusText;

	/** The sub status text (if any) */
	FText SubStatusText;

	/** The asset registry's asset discovery progress as a percentage */
	TOptional<float> Progress;

	/** The current wrap point for the status text */
	float StatusTextWrapWidth;

	/** The way the indicator will scale in/out before/after displaying progress */
	EAssetDiscoveryIndicatorScaleMode::Type ScaleMode;

	/** The fade in/out animation */
	FCurveSequence FadeAnimation;
	FCurveHandle FadeCurve;
	FCurveHandle ScaleCurve;
};
