// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssetDropTarget.h"

#include "AssetSelection.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/DragAndDrop.h"
#include "Styling/AppStyle.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

struct FGeometry;

#define LOCTEXT_NAMESPACE "EditorWidgets"

void SAssetDropTarget::Construct(const FArguments& InArgs )
{
	OnAssetsDropped = InArgs._OnAssetsDropped;
	OnAreAssetsAcceptableForDrop = InArgs._OnAreAssetsAcceptableForDrop;
	OnAreAssetsAcceptableForDropWithReason = InArgs._OnAreAssetsAcceptableForDropWithReason;
	bSupportsMultiDrop = InArgs._bSupportsMultiDrop;

	SDropTarget::Construct(
		SDropTarget::FArguments()
		.bOnlyRecognizeOnDragEnter(InArgs._bOnlyRecognizeOnDragEnter)
		.OnDropped(this, &SAssetDropTarget::OnDropped)
		[
			InArgs._Content.Widget
		]);
}

FReply SAssetDropTarget::OnDropped(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	if (OnAssetsDropped.IsBound())
	{
		bool bRecongnizedEvent = false;
		TArray<FAssetData> AssetDatas = GetDroppedAssets(InDragDropEvent.GetOperation(), bRecongnizedEvent);

		if (bRecongnizedEvent)
		{
			OnAssetsDropped.Execute(InDragDropEvent, AssetDatas);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

bool SAssetDropTarget::OnAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation) const
{
	bool bRecongnizedEvent = false;
	TArray<FAssetData> AssetDatas = GetDroppedAssets(DragDropOperation, bRecongnizedEvent);

	if (bRecongnizedEvent)
	{
		// Check and see if its valid to drop this object
		if (OnAreAssetsAcceptableForDropWithReason.IsBound())
		{
			FText FailureReason;
			if (OnAreAssetsAcceptableForDropWithReason.Execute(AssetDatas, FailureReason))
			{
				return true;
			}
			else
			{
				if (IsDragOver() && !FailureReason.IsEmpty())
				{
					if (DragDropOperation.IsValid() && DragDropOperation->IsOfType<FDecoratedDragDropOp>())
					{
						TSharedPtr<FDecoratedDragDropOp> DragDropOp = StaticCastSharedPtr<FDecoratedDragDropOp>(DragDropOperation);
						if (DragDropOp.IsValid())
						{
							DragDropOp->SetToolTip(FailureReason, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
						}
					}
				}

				return false;
			}
		}
		else if (OnAreAssetsAcceptableForDrop.IsBound())
		{
			return OnAreAssetsAcceptableForDrop.Execute(AssetDatas);
		}
		else
		{
			// If no delegate is bound assume its always valid to drop this object
			return true;
		}
	}

	return false;
}

bool SAssetDropTarget::OnIsRecognized(TSharedPtr<FDragDropOperation> DragDropOperation) const
{
	bool bRecognizedEvent = false;
	GetDroppedAssets(DragDropOperation, bRecognizedEvent);

	return bRecognizedEvent;
}

void SAssetDropTarget::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	SDropTarget::OnDragLeave(DragDropEvent);

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (Operation.IsValid() && Operation->IsOfType<FDecoratedDragDropOp>())
	{
		TSharedPtr<FDecoratedDragDropOp> DragDropOp = StaticCastSharedPtr<FDecoratedDragDropOp>(Operation);
		DragDropOp->ResetToDefaultToolTip();
	}
}

TArray<FAssetData> SAssetDropTarget::GetDroppedAssets(TSharedPtr<FDragDropOperation> DragDropOperation, bool& bOutRecognizedEvent) const
{
	TArray<FAssetData> DroppedAssets;
	if (!DragDropOperation)
	{
		return DroppedAssets;
	}
	
	if ( DragDropOperation->IsOfType<FActorDragDropOp>() )
	{
		// Handle actors being dragged
		TSharedPtr<FActorDragDropOp> ActorDragDrop = StaticCastSharedPtr<FActorDragDropOp>(DragDropOperation);

		for (TWeakObjectPtr<AActor> Actor : ActorDragDrop->Actors)
		{
			FAssetData DroppedActorAsset(Actor.Get());
			if (DroppedActorAsset.IsValid())
			{
				DroppedAssets.Emplace(Actor.Get());
			}
		}
	}
	else
	{
		// Handle assets being dragged
		DroppedAssets = AssetUtil::ExtractAssetDataFromDrag(DragDropOperation);
	}

	bOutRecognizedEvent = (bSupportsMultiDrop && (DroppedAssets.Num() > 0)) || (DroppedAssets.Num() == 1);
	return DroppedAssets;
}

#undef LOCTEXT_NAMESPACE
