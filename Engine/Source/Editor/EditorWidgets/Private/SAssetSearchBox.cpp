// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssetSearchBox.h"

#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Views/ITypedTableView.h"
#include "IDocumentation.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/LocKeyFuncs.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Tuple.h"
#include "Types/SlateStructs.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;
class SWidget;
struct FGeometry;

/** Case sensitive hashing function for TMap */
template <typename ValueType>
struct FAssetSearchCategoryKeyMapFuncs : BaseKeyFuncs<ValueType, FText, /*bInAllowDuplicateKeys*/false>
{
	static FORCEINLINE const FString& GetSourceString(const FText& InText)
	{
		const FString* SourceString = FTextInspector::GetSourceString(InText);
		check(SourceString);
		return *SourceString;
	}
	static FORCEINLINE const FText& GetSetKey(const TPair<FText, ValueType>& Element)
	{
		return Element.Key;
	}
	static FORCEINLINE bool Matches(const FText& A, const FText& B)
	{
		return GetSourceString(A).Equals(GetSourceString(B), ESearchCase::CaseSensitive);
	}
	static FORCEINLINE uint32 GetKeyHash(const FText& Key)
	{
		return FLocKey::ProduceHash(GetSourceString(Key));
	}
};

void SAssetSearchBox::Construct( const FArguments& InArgs )
{
	OnTextChanged = InArgs._OnTextChanged;
	OnTextCommitted = InArgs._OnTextCommitted;
	OnKeyDownHandler = InArgs._OnKeyDownHandler;
	PossibleSuggestions = InArgs._PossibleSuggestions;
	OnAssetSearchBoxSuggestionFilter = InArgs._OnAssetSearchBoxSuggestionFilter;
	OnAssetSearchBoxSuggestionChosen = InArgs._OnAssetSearchBoxSuggestionChosen;
	PreCommittedText = InArgs._InitialText.Get();
	bMustMatchPossibleSuggestions = InArgs._MustMatchPossibleSuggestions.Get();

	if (!OnAssetSearchBoxSuggestionFilter.IsBound())
	{
		OnAssetSearchBoxSuggestionFilter.BindStatic(&SAssetSearchBox::DefaultSuggestionFilterImpl);
	}

	if (!OnAssetSearchBoxSuggestionChosen.IsBound())
	{
		OnAssetSearchBoxSuggestionChosen.BindStatic(&SAssetSearchBox::DefaultSuggestionChosenImpl);
	}

	ChildSlot
		[
			SAssignNew(SuggestionBox, SMenuAnchor)
			.Placement( InArgs._SuggestionListPlacement )
			[
				/* Use an SFilterSearchBox internally to add the ability to show search history and potentially
				 * save searches as filters if used with a Filter Bar widget (@see SBasicFilterBar etc)
				 */
				SAssignNew(InputText, SFilterSearchBox)
				.InitialText(InArgs._InitialText)
				.HintText(InArgs._HintText)
				.OnTextChanged(this, &SAssetSearchBox::HandleTextChanged)
				.OnTextCommitted(this, &SAssetSearchBox::HandleTextCommitted)
				.DelayChangeNotificationsWhileTyping( InArgs._DelayChangeNotificationsWhileTyping )
				.OnKeyDownHandler(this, &SAssetSearchBox::HandleKeyDown)
				.ShowSearchHistory(InArgs._ShowSearchHistory)
				.OnSaveSearchClicked(InArgs._OnSaveSearchClicked)
				.ExtraWidget(InArgs._ExtraWidget)
			]
			.MenuContent
			(
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				.Padding( FMargin(2.f) )
				[
					SNew(SBox)
					.WidthOverride(this, &SAssetSearchBox::GetSuggestionListWidth)
					.HeightOverride(InArgs._SuggestionListHeight) // avoids flickering, ideally this would be adaptive to the content without flickering
					[
						SAssignNew(SuggestionListView, SListView< TSharedPtr<FSuggestionListEntry> >)
						.ListItemsSource(&Suggestions)
						.SelectionMode( ESelectionMode::Single )							// Ideally the mouse over would not highlight while keyboard controls the UI
						.OnGenerateRow(this, &SAssetSearchBox::MakeSuggestionListItemWidget)
						.OnSelectionChanged( this, &SAssetSearchBox::OnSelectionChanged)
						.ScrollbarDragFocusCause(EFocusCause::SetDirectly) // Use SetDirect so that clicking the scrollbar doesn't close the suggestions list
					]
				]
			)
		];

	// Set Search Tooltip
	if (InArgs._ToolTipText.IsSet())
	{
		InputText->SetToolTipText(InArgs._ToolTipText);
	}
	else if (InArgs._ToolTip.IsSet())
	{
		InputText->SetToolTip(InArgs._ToolTip);
	}
	else
	{
		InputText->SetToolTip(IDocumentation::Get()->CreateToolTip(NSLOCTEXT("AssetSearchBox", "FilterSearchHint", "Type here to search"), /*OverrideContent = */nullptr, "Shared/ContentBrowser", "AdvancedSearchSyntax"));
	}
}

void SAssetSearchBox::SetText(const TAttribute< FText >& InNewText)
{
	InputText->SetText(InNewText);
	PreCommittedText = InNewText.Get();
}

FText SAssetSearchBox::GetText() const
{
	return InputText->GetText();
}

void SAssetSearchBox::SetError( const FText& InError )
{
	InputText->SetError(InError);
}

void SAssetSearchBox::SetError( const FString& InError )
{
	InputText->SetError(InError);
}

FReply SAssetSearchBox::OnPreviewKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if ( SuggestionBox->IsOpen() && InKeyEvent.GetKey() == EKeys::Escape )
	{
		// Clear any selection first to prevent the currently selection being set in the text box
		SuggestionListView->ClearSelection();
		SuggestionBox->SetIsOpen(false, false);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SAssetSearchBox::HandleKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if ( SuggestionBox->IsOpen() && (InKeyEvent.GetKey() == EKeys::Up || InKeyEvent.GetKey() == EKeys::Down) )
	{
		const bool bSelectingUp = InKeyEvent.GetKey() == EKeys::Up;
		TSharedPtr<FSuggestionListEntry> SelectedSuggestion = GetSelectedSuggestion();

		int32 TargetIdx = INDEX_NONE;
		if ( SelectedSuggestion.IsValid() )
		{
			const int32 SelectionDirection = bSelectingUp ? -1 : 1;

			// Select the next non-header suggestion, based on the direction of travel
			TargetIdx = Suggestions.IndexOfByKey(SelectedSuggestion);
			if (Suggestions.IsValidIndex(TargetIdx))
			{
				do
				{
					TargetIdx += SelectionDirection;
				}
				while (Suggestions.IsValidIndex(TargetIdx) && Suggestions[TargetIdx]->bIsHeader);
			}
		}
		else if ( !bSelectingUp && Suggestions.Num() > 0 )
		{
			// Nothing selected and pressed down, select the first non-header suggestion
			TargetIdx = 0;
			while (Suggestions.IsValidIndex(TargetIdx) && Suggestions[TargetIdx]->bIsHeader)
			{
				TargetIdx += 1;
			}
		}

		if (Suggestions.IsValidIndex(TargetIdx))
		{
			SuggestionListView->SetSelection(Suggestions[TargetIdx]);
			SuggestionListView->RequestScrollIntoView(Suggestions[TargetIdx]);
		}

		return FReply::Handled();
	}

	if (OnKeyDownHandler.IsBound())
	{
		return OnKeyDownHandler.Execute(MyGeometry, InKeyEvent);
	}

	return FReply::Unhandled();
}

bool SAssetSearchBox::SupportsKeyboardFocus() const
{
	return InputText->SupportsKeyboardFocus();
}

bool SAssetSearchBox::HasKeyboardFocus() const
{
	// Since keyboard focus is forwarded to our editable text, we will test it instead
	return InputText->HasKeyboardFocus();
}

FReply SAssetSearchBox::OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent )
{
	// Forward keyboard focus to our editable text widget
	return InputText->OnFocusReceived(MyGeometry, InFocusEvent);
}

void SAssetSearchBox::HandleTextChanged(const FText& NewText)
{
	OnTextChanged.ExecuteIfBound(NewText);
	UpdateSuggestionList();
}

void SAssetSearchBox::HandleTextCommitted(const FText& NewText, ETextCommit::Type CommitType)
{
	TSharedPtr<FSuggestionListEntry> SelectedSuggestion = GetSelectedSuggestion();

	bool bCommitText = true;
	FText CommittedText;
	if ( SelectedSuggestion.IsValid() && !SelectedSuggestion->bIsHeader && CommitType != ETextCommit::OnCleared )
	{
		// Pressed selected a suggestion, set the text
		CommittedText = OnAssetSearchBoxSuggestionChosen.Execute(NewText, SelectedSuggestion->Suggestion);
	}
	else
	{
		if ( CommitType == ETextCommit::OnCleared )
		{
			// Clear text when escape is pressed then commit an empty string
			CommittedText = FText::GetEmpty();
		}
		else if (bMustMatchPossibleSuggestions && PossibleSuggestions.Get().ContainsByPredicate([this, NewTextStr = NewText.ToString()](const FAssetSearchBoxSuggestion& InSuggestion) { return InSuggestion.SuggestionString == NewTextStr; }))
		{
			// If the text is a suggestion, set the text.
			CommittedText = NewText;
		}
		else if( bMustMatchPossibleSuggestions )
		{
			// commit the original text if we have to match a suggestion
			CommittedText = PreCommittedText;
		}
		else
		{
			// otherwise, set the typed text
			CommittedText = NewText;
		}	
	}

	// Set the text and execute the delegate
	SetText(CommittedText);
	OnTextCommitted.ExecuteIfBound(CommittedText, CommitType);

	if(CommitType != ETextCommit::Default)
	{
		// Clear the suggestion box if the user has navigated away or set their own text.
		SuggestionBox->SetIsOpen(false, false);
	}
}

void SAssetSearchBox::OnSelectionChanged( TSharedPtr<FSuggestionListEntry> NewValue, ESelectInfo::Type SelectInfo )
{
	// If the user clicked directly on an item to select it, then accept the choice and close the window
	if(SelectInfo == ESelectInfo::OnMouseClick && !NewValue->bIsHeader)
	{
		const FText SearchText = InputText->GetText();
		const FText NewText = OnAssetSearchBoxSuggestionChosen.Execute(SearchText, NewValue->Suggestion);
		SetText(NewText);
		SuggestionBox->SetIsOpen(false, false);
		FocusEditBox();		
	}
}

TSharedRef<ITableRow> SAssetSearchBox::MakeSuggestionListItemWidget(TSharedPtr<FSuggestionListEntry> Suggestion, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(Suggestion.IsValid());
	check(Suggestions.Num() > 0);

	const bool bIsFirstItem = Suggestions[0] == Suggestion;
	const bool bIdentItems = Suggestions[0]->bIsHeader;

	TSharedPtr<SWidget> RowWidget;
	if (Suggestion->bIsHeader)
	{
		TSharedRef<SVerticalBox> HeaderVBox = SNew(SVerticalBox);

		if (!bIsFirstItem)
		{
			HeaderVBox->AddSlot()
			.AutoHeight()
			.Padding(0.0f, 4.0f, 0.0f, 2.0f) // Add some empty space before the line, and a tiny bit after it
			[
				SNew(SBorder)
				.Padding(FAppStyle::GetMargin("Menu.Separator.Padding")) // We'll use the border's padding to actually create the horizontal line
				.BorderImage(FAppStyle::GetBrush("Menu.Separator"))
			];
		}

		HeaderVBox->AddSlot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(Suggestion->DisplayName.ToUpper())
			.TextStyle(FAppStyle::Get(), "Menu.Heading")
		];

		RowWidget = HeaderVBox;
	}
	else
	{
		RowWidget =
			SNew(SBox)
			.Padding(FAppStyle::GetMargin(bIdentItems ? "Menu.Block.IndentedPadding" : "Menu.Block.Padding"))
			[
				SNew(STextBlock)
				.Text(Suggestion->DisplayName)
				.HighlightText(this, &SAssetSearchBox::GetHighlightText)
			];
	}

	return
		SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
		[
			RowWidget.ToSharedRef()
		];
}

FText SAssetSearchBox::GetHighlightText() const
{
	return SuggestionHighlightText;
}

void SAssetSearchBox::UpdateSuggestionList()
{
	const FText SearchText = InputText->GetText();

	Suggestions.Reset();
	SuggestionHighlightText = FText::GetEmpty();

	if (!SearchText.IsEmpty())
	{
		typedef TMap<FText, TArray<TSharedPtr<FSuggestionListEntry>>, FDefaultSetAllocator, FAssetSearchCategoryKeyMapFuncs<TArray<TSharedPtr<FSuggestionListEntry>>>> FCategorizedSuggestionsMap;

		// Get the potential suggestions and run them through the filter
		TArray<FAssetSearchBoxSuggestion> FilteredSuggestions = PossibleSuggestions.Get();
		OnAssetSearchBoxSuggestionFilter.Execute(SearchText, FilteredSuggestions, SuggestionHighlightText);

		// Split the suggestions list into categories
		FCategorizedSuggestionsMap CategorizedSuggestions;
		for (const FAssetSearchBoxSuggestion& Suggestion : FilteredSuggestions)
		{
			TArray<TSharedPtr<FSuggestionListEntry>>& CategorySuggestions = CategorizedSuggestions.FindOrAdd(Suggestion.CategoryName);
			CategorySuggestions.Add(MakeShared<FSuggestionListEntry>(FSuggestionListEntry{ Suggestion.SuggestionString, Suggestion.DisplayName, false }));
		}

		// Rebuild the flat list in categorized groups
		// If there is only one category, and that category is empty (undefined), then skip adding the category headers
		const bool bSkipCategoryHeaders = CategorizedSuggestions.Num() == 1 && CategorizedSuggestions.Contains(FText::GetEmpty());
		for (const auto& CategorySuggestionsPair : CategorizedSuggestions)
		{
			if (!bSkipCategoryHeaders)
			{
				const FText CategoryDisplayName = CategorySuggestionsPair.Key.IsEmpty() ? NSLOCTEXT("AssetSearchBox", "UndefinedCategory", "Undefined") : CategorySuggestionsPair.Key;
				Suggestions.Add(MakeShared<FSuggestionListEntry>(FSuggestionListEntry{ TEXT(""), CategoryDisplayName, true }));
			}
			Suggestions.Append(CategorySuggestionsPair.Value);
		}
	}

	if (Suggestions.Num() > 0 && HasKeyboardFocus())
	{
		// At least one suggestion was found, open the menu
		SuggestionBox->SetIsOpen(true, false);
	}
	else
	{
		// No suggestions were found, close the menu
		SuggestionBox->SetIsOpen(false, false);
	}

	SuggestionListView->RequestListRefresh();
}

void SAssetSearchBox::FocusEditBox()
{
	FWidgetPath WidgetToFocusPath;
	FSlateApplication::Get().GeneratePathToWidgetUnchecked( InputText.ToSharedRef(), WidgetToFocusPath );
	FSlateApplication::Get().SetKeyboardFocus( WidgetToFocusPath, EFocusCause::SetDirectly );
}

TSharedPtr<SAssetSearchBox::FSuggestionListEntry> SAssetSearchBox::GetSelectedSuggestion() const
{
	TSharedPtr<FSuggestionListEntry> SelectedSuggestion;
	if ( SuggestionBox->IsOpen() )
	{
		const TArray< TSharedPtr<FSuggestionListEntry> >& SelectedSuggestionList = SuggestionListView->GetSelectedItems();
		if ( SelectedSuggestionList.Num() > 0 )
		{
			// Selection mode is Single, so there should only be one suggestion at the most
			SelectedSuggestion = SelectedSuggestionList[0];
		}
	}

	return SelectedSuggestion;
}

void SAssetSearchBox::DefaultSuggestionFilterImpl(const FText& SearchText, TArray<FAssetSearchBoxSuggestion>& PossibleSuggestions, FText& SuggestionHighlightText)
{
	// Default implementation just filters against the current search text
	PossibleSuggestions.RemoveAll([SearchStr = SearchText.ToString()](const FAssetSearchBoxSuggestion& InSuggestion)
	{
		return !InSuggestion.SuggestionString.Contains(SearchStr);
	});

	SuggestionHighlightText = SearchText;
}

FText SAssetSearchBox::DefaultSuggestionChosenImpl(const FText& SearchText, const FString& Suggestion)
{
	// Default implementation just uses the suggestion as the search text
	return FText::FromString(Suggestion);
}

FOptionalSize SAssetSearchBox::GetSuggestionListWidth() const
{
	// to enforce some minimum width, ideally we define the minimum, not a fixed width
	constexpr float MinWidth = 175.f;

	if (!InputText.IsValid())
	{
		return MinWidth;
	}

	return FMath::Max(InputText->GetCachedGeometry().GetLocalSize().X, MinWidth);
}

void SAssetSearchBox::SetOnSaveSearchHandler(SFilterSearchBox::FOnSaveSearchClicked InOnSaveSearchHandler)
{
	InputText->SetOnSaveSearchHandler(InOnSaveSearchHandler);
}
