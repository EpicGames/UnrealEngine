// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDropTarget.h"

#include "Framework/Application/SlateApplication.h"
#include "Layout/Children.h"
#include "Layout/Clipping.h"
#include "Layout/Geometry.h"
#include "Math/Vector2D.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "SlotBase.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SOverlay.h"

class FSlateRect;
class FWidgetStyle;


#define LOCTEXT_NAMESPACE "EditorWidgets"

void SDropTarget::Construct(const FArguments& InArgs)
{
	DroppedEvent = InArgs._OnDropped;
	AllowDropEvent = InArgs._OnAllowDrop;
	IsRecognizedEvent = InArgs._OnIsRecognized;
	OnDragEnterEvent = InArgs._OnDragEnter;
	OnDragLeaveEvent = InArgs._OnDragLeave;
	bOnlyRecognizeOnDragEnter = InArgs._bOnlyRecognizeOnDragEnter;
	bUseAllowDropCache = InArgs._bUseAllowDropCache;
	
	bIsDragEventRecognized = false;
	bAllowDrop = false;
	bIsDragOver = false;

	ValidColor = InArgs._ValidColor;
	InvalidColor = InArgs._InvalidColor;

	VerticalImage = InArgs._VerticalImage;
	HorizontalImage = InArgs._HorizontalImage;

	// if we want to use the cache, we need to detect whether to clear the cache on tick
	SetCanTick(bUseAllowDropCache);
	
	ChildSlot
	[
		SNew(SOverlay)
		.Clipping(EWidgetClipping::ClipToBounds)
		+ SOverlay::Slot()
		[
			InArgs._Content.Widget
		]

		+ SOverlay::Slot()
		[
			SNew(SBorder)
			.Visibility(this, &SDropTarget::GetDragOverlayVisibility)
			.BorderImage(InArgs._BackgroundImage)
			.BorderBackgroundColor(this, &SDropTarget::GetBackgroundBrightness)
		]
	];
}

FSlateColor SDropTarget::GetBackgroundBrightness() const
{
	return bAllowDrop ? ValidColor : InvalidColor;
}

EVisibility SDropTarget::GetDragOverlayVisibility() const
{
	if ( FSlateApplication::Get().IsDragDropping() )
	{
		bool bCheckForAllowDrop = true;

		if(bOnlyRecognizeOnDragEnter.Get() && !bIsDragOver)
		{
			bCheckForAllowDrop = false;
		}
		
		if(bCheckForAllowDrop)
		{
			if(AllowDrop(FSlateApplication::Get().GetDragDroppingContent()) || (bIsDragOver && bIsDragEventRecognized))
			{
				return EVisibility::HitTestInvisible;
			}
		}
	}

	return EVisibility::Hidden;
}

FReply SDropTarget::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Handle the reply if we are allowed to drop, otherwise do not handle it.
	return AllowDrop(DragDropEvent.GetOperation()) ? FReply::Handled() : FReply::Unhandled();
}

bool SDropTarget::AllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation) const
{
	if(bUseAllowDropCache && AllowDropCache.IsSet())
	{
		bAllowDrop = AllowDropCache.GetValue();
	}
	else
	{
		bAllowDrop = OnAllowDrop(DragDropOperation);

		if(bUseAllowDropCache)
		{
			AllowDropCache = bAllowDrop;
		}
	}
	bIsDragEventRecognized = OnIsRecognized(DragDropOperation) || bAllowDrop;
	return bAllowDrop;
}

bool SDropTarget::OnAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation) const
{
	if ( AllowDropEvent.IsBound() )
	{
		return AllowDropEvent.Execute(DragDropOperation);
	}

	return false;
}

bool SDropTarget::OnIsRecognized(TSharedPtr<FDragDropOperation> DragDropOperation) const
{
	if ( IsRecognizedEvent.IsBound() )
	{
		return IsRecognizedEvent.Execute(DragDropOperation);
	}

	return false;
}

void SDropTarget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if(bUseAllowDropCache)
	{
		bWasDragDroppingLastFrame = bIsDragDropping;
		bIsDragDropping = FSlateApplication::Get().IsDragDropping();

		if(bIsDragDropping && !bWasDragDroppingLastFrame)
		{
			ClearAllowDropCache();
		}
	}
}

FReply SDropTarget::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	const bool bCurrentbAllowDrop = bAllowDrop;

	// We've dropped an asset so we are no longer being dragged over
	bIsDragEventRecognized = false;
	bIsDragOver = false;
	bAllowDrop = false;

	// if we allow drop, call a delegate to handle the drop
	if ( bCurrentbAllowDrop )
	{
		if ( DroppedEvent.IsBound() )
		{
			return DroppedEvent.Execute(MyGeometry, DragDropEvent);
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SDropTarget::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// initially we dont recognize this event
	bIsDragEventRecognized = false;
	bIsDragOver = true;

	OnDragEnterEvent.ExecuteIfBound(DragDropEvent);
}

void SDropTarget::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	// No longer being dragged over
	bIsDragEventRecognized = false;
	// Disallow dropping if not dragged over.
	bAllowDrop = false;

	bIsDragOver = false;
	OnDragLeaveEvent.ExecuteIfBound(DragDropEvent);
}

int32 SDropTarget::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	if ( GetDragOverlayVisibility().IsVisible() )
	{
		if ( bIsDragEventRecognized )
		{
			FSlateColor DashColor = bAllowDrop ? ValidColor : InvalidColor;

			int32 DashLayer = LayerId + 1;
			
			const float Inset = 3.0f;

			// Top
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				DashLayer,
				AllottedGeometry.ToPaintGeometry(FVector2f(AllottedGeometry.GetLocalSize().X-Inset*2, HorizontalImage->ImageSize.Y), FSlateLayoutTransform(FVector2f(Inset, 0))),
				HorizontalImage,
				ESlateDrawEffect::None,
				DashColor.GetColor(InWidgetStyle));

			// Bottom
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				DashLayer,
				AllottedGeometry.ToPaintGeometry(FVector2f(AllottedGeometry.Size.X-Inset * 2, HorizontalImage->ImageSize.Y), FSlateLayoutTransform(FVector2f(Inset, AllottedGeometry.GetLocalSize().Y - HorizontalImage->ImageSize.Y))),
				HorizontalImage,
				ESlateDrawEffect::None,
				DashColor.GetColor(InWidgetStyle));

			// Left
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				DashLayer,
				AllottedGeometry.ToPaintGeometry(FVector2f(VerticalImage->ImageSize.X, AllottedGeometry.GetLocalSize().Y-Inset * 2), FSlateLayoutTransform(FVector2f(0, Inset))),
				VerticalImage,
				ESlateDrawEffect::None,
				DashColor.GetColor(InWidgetStyle));

			// Right
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				DashLayer,
				AllottedGeometry.ToPaintGeometry(FVector2f(VerticalImage->ImageSize.X, AllottedGeometry.GetLocalSize().Y-Inset * 2), FSlateLayoutTransform(FVector2f(AllottedGeometry.GetLocalSize().X - VerticalImage->ImageSize.X, Inset))),
				VerticalImage,
				ESlateDrawEffect::None,
				DashColor.GetColor(InWidgetStyle));

			return DashLayer;
		}
	}

	return LayerId;
}

#undef LOCTEXT_NAMESPACE
