// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEnumCombo.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "EditorWidgets"

void SEnumComboBox::Construct(const FArguments& InArgs, const UEnum* InEnum)
{
	static const FName UseEnumValuesAsMaskValuesInEditorName(TEXT("UseEnumValuesAsMaskValuesInEditor"));
	
	Enum = InEnum;
	CurrentValue = InArgs._CurrentValue;
	check(CurrentValue.IsBound());
	OnEnumSelectionChangedDelegate = InArgs._OnEnumSelectionChanged;
	OnGetToolTipForValue = InArgs._OnGetToolTipForValue;
	Font = InArgs._Font;
	OverrideNoFlagsSetText = InArgs._OverrideNoFlagsSetText;
	bUpdatingSelectionInternally = false;
	bIsBitflagsEnum = InArgs._bForceBitFlags || Enum->HasMetaData(TEXT("Bitflags"));

	TArray<int32> EnumValueSubset = InArgs._EnumValueSubset;
	if(EnumValueSubset.IsEmpty())
	{
		EnumValueSubset.Reserve(Enum->NumEnums() - 1);
		for (int32 i = 0; i < Enum->NumEnums() - 1; i++)
		{
			// Note: SEnumComboBox API prior to bitflags only supports 32 bit values, truncating the value here to keep the old API.
			const int32 Value = static_cast<int32>(Enum->GetValueByIndex(i));
			EnumValueSubset.Add(Value);
		}
	}

	if (bIsBitflagsEnum)
	{
		const bool bUseEnumValuesAsMaskValues = Enum->GetBoolMetaData(UseEnumValuesAsMaskValuesInEditorName);
		const int32 BitmaskBitCount = sizeof(int32) << 3;

		for (int32 i = 0; i < EnumValueSubset.Num(); i++)
		{
			int32 Value = EnumValueSubset[i];
			const int32 Index = Enum->GetIndexByValue(Value);
			const bool bIsHidden = Enum->HasMetaData(TEXT("Hidden"), Index);
			if (Value >= 0 && !bIsHidden)
			{
				if (bUseEnumValuesAsMaskValues)
				{
					if (Value >= MAX_int32 || !FMath::IsPowerOfTwo(Value))
					{
						continue;
					}
				}
				else
				{
					if (Value >= BitmaskBitCount)
					{
						continue;
					}
					Value = 1 << Value;
				}

				FText DisplayName = Enum->GetDisplayNameTextByIndex(Index);
				FText TooltipText = Enum->GetToolTipTextByIndex(Index);
				if (TooltipText.IsEmpty())
				{
					TooltipText = FText::Format(LOCTEXT("BitmaskDefaultFlagToolTipText", "Toggle {0} on/off"), DisplayName);
				}

				VisibleEnums.Emplace(Index, Value, DisplayName, TooltipText);
			}
		}
	}
	else
	{
		for (int32 i = 0; i < EnumValueSubset.Num(); i++)
		{
			const int32 Value = EnumValueSubset[i];
			const int32 Index = Enum->GetIndexByValue(Value);
			if (Enum->HasMetaData(TEXT("Hidden"), Index) == false)
			{
				VisibleEnums.Emplace(Index, static_cast<int32>(Enum->GetValueByIndex(Index)), Enum->GetDisplayNameTextByIndex(Index), Enum->GetToolTipTextByIndex(Index));
			}
		}
	}

	SComboButton::Construct(SComboButton::FArguments()
		.ComboButtonStyle(InArgs._ComboButtonStyle)
		.ButtonStyle(InArgs._ButtonStyle)
		.ContentPadding(InArgs._ContentPadding)
		.OnGetMenuContent(this, &SEnumComboBox::OnGetMenuContent)
		.ButtonContent()
		[
			SNew(STextBlock)
			.TextStyle(InArgs._TextStyle)
			.Font(Font)
			.Text(this, &SEnumComboBox::GetCurrentValueText)
			.ToolTipText(this, &SEnumComboBox::GetCurrentValueTooltip)
		]);
}


FText SEnumComboBox::GetCurrentValueText() const
{
	if (bIsBitflagsEnum)
	{
		if (CurrentValue.IsSet())
		{
			const int32 BitmaskValue = CurrentValue.Get();
			if (BitmaskValue != 0)
			{
				TArray<FText> SetFlags;
				SetFlags.Reserve(VisibleEnums.Num());

				for (const FEnumInfo& FlagInfo : VisibleEnums)
				{
					if ((BitmaskValue & FlagInfo.Value) != 0)
					{
						SetFlags.Add(FlagInfo.DisplayName);
					}
				}
				if (SetFlags.Num() > 3)
				{
					SetFlags.SetNum(3);
					SetFlags.Add(FText::FromString("..."));
				}

				return FText::Join(FText::FromString(" | "), SetFlags);
			}
			if (OverrideNoFlagsSetText.IsSet())
			{
				return OverrideNoFlagsSetText.Get();
			}
			else
			{
				return LOCTEXT("BitmaskButtonContentNoFlagsSet", "(No Flags Set)");
			}
		}
		return FText::GetEmpty();
	}

	const int32 ValueNameIndex = Enum->GetIndexByValue(CurrentValue.Get());
	return Enum->GetDisplayNameTextByIndex(ValueNameIndex);
}

FText SEnumComboBox::GetCurrentValueTooltip() const
{
	if (bIsBitflagsEnum)
	{
		const int32 BitmaskValue = CurrentValue.Get();
		if (BitmaskValue != 0)
		{
			TArray<FText> SetFlags;
			SetFlags.Reserve(VisibleEnums.Num());

			for (const FEnumInfo& FlagInfo : VisibleEnums)
			{
				if ((BitmaskValue & FlagInfo.Value) != 0)
				{
					if (OnGetToolTipForValue.IsBound())
					{
						SetFlags.Add(OnGetToolTipForValue.Execute(FlagInfo.Index));
					}
					else
					{
						SetFlags.Add(FlagInfo.DisplayName);
					}
				}
			}

			return FText::Join(FText::FromString(" | "), SetFlags);
		}
		return FText::GetEmpty();
	}
	
	const int32 ValueNameIndex = Enum->GetIndexByValue(CurrentValue.Get());
	if (OnGetToolTipForValue.IsBound())
	{
		return OnGetToolTipForValue.Execute(ValueNameIndex);
	}
	return Enum->GetToolTipTextByIndex(ValueNameIndex);
}

TSharedRef<SWidget> SEnumComboBox::OnGetMenuContent()
{
	const bool bCloseAfterSelection = !bIsBitflagsEnum;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr, nullptr, true);

	for (const FEnumInfo& FlagInfo : VisibleEnums)
	{
		MenuBuilder.AddMenuEntry(
			FlagInfo.DisplayName,
			FlagInfo.TooltipText,
			FSlateIcon(),
			FUIAction
			(
				FExecuteAction::CreateLambda([this, FlagInfo]()
				{
					if (bIsBitflagsEnum)
					{
						// Toggle value
						const int32 Value = CurrentValue.Get() ^ FlagInfo.Value;
						OnEnumSelectionChangedDelegate.ExecuteIfBound(Value, ESelectInfo::Direct);
					}
					else
					{
						// Set value
						OnEnumSelectionChangedDelegate.ExecuteIfBound(FlagInfo.Value, ESelectInfo::OnMouseClick);
					}
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, FlagInfo]() -> bool
				{
					return (CurrentValue.Get() & FlagInfo.Value) != 0;
				})
			),
			NAME_None,
			bIsBitflagsEnum ? EUserInterfaceActionType::Check : EUserInterfaceActionType::None);
	}

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
