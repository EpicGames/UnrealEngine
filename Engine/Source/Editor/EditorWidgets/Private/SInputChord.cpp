// Copyright Epic Games, Inc. All Rights Reserved.

#include "SInputChord.h"

#include "Components/HorizontalBox.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SInputChord::Construct(const FArguments& InArgs)
{
	// Padding between Icon and InputLabel, and between either side of the InputLabel delimiter
	static constexpr float HorizontalElementPadding = 5.0f;
	static constexpr float DefaultVerticalPadding = 2.0f;

	TSharedRef<SHorizontalBox> ContainerWidget = SNew(SHorizontalBox);

	// Prepend Icon, if set
	if (InArgs._Icon->IsSet())
	{
		ContainerWidget->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(FMargin{0.0f, DefaultVerticalPadding, HorizontalElementPadding, DefaultVerticalPadding})
		[
			SNew(SImage)
			.Image(InArgs._Icon)
		];
	}

	// Always display the input label (key combo)
	ContainerWidget->AddSlot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.TextStyle(InArgs._InputLabelStyle)
		.Text(InArgs._InputLabel)
	];

	// Only add the action label and delimiter if the action label is set
	if (!InArgs._ActionLabel.IsEmpty())
	{
		ContainerWidget->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(FMargin{HorizontalElementPadding, DefaultVerticalPadding})
		[
			InArgs._InputLabelDelimiterOverride.IsValid()
			? InArgs._InputLabelDelimiterOverride.ToSharedRef()
			: SNew(SSeparator)
			.ColorAndOpacity(FLinearColor{1,1,1, 0.4f})
			.SeparatorImage(FAppStyle::Get().GetBrush("WhiteBrush"))
			.Orientation(EOrientation::Orient_Vertical)
			.Thickness(1.0f)
		];

		ContainerWidget->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.TextStyle(InArgs._ActionLabelStyle)
			.Text(InArgs._ActionLabel)
		];
	}

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		ContainerWidget
	];
}

TSharedRef<SInputChord> SInputChord::MakeForInputChord(
	const FInputChord& InInputChord,
	const FText& InActionLabel)
{
	return SNew(SInputChord)
		.Icon(GetIconForInputChord(InInputChord))
		.InputLabel(InInputChord.GetInputText())
		.ActionLabel(InActionLabel);
}

TSharedRef<SInputChord> SInputChord::MakeForCommandInfo(const FUICommandInfo& InCommandInfo, const EMultipleKeyBindingIndex InKeyBindingIndex)
{
	const FSlateBrush* IconBrush = nullptr;

	if (ensureMsgf(
		InCommandInfo.GetActiveChord(InKeyBindingIndex)->IsValidChord(),
		TEXT("CommandInfo '%s' has no valid input chord"), *InCommandInfo.GetLabel().ToString()))
	{
		IconBrush = GetIconForInputChord(*InCommandInfo.GetActiveChord(InKeyBindingIndex));
	}

	return SNew(SInputChord)
		.Icon(IconBrush)
		.InputLabel(InCommandInfo.GetInputText())
		.ActionLabel(InCommandInfo.GetLabel())
		.ToolTipText(InCommandInfo.GetDescription());
}

const FSlateBrush* SInputChord::GetIconForInputChord(const FInputChord& InInputChord)
{
	return FAppStyle::GetBrush(EKeys::GetMenuCategoryPaletteIcon(InInputChord.Key.GetMenuCategory()));
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
