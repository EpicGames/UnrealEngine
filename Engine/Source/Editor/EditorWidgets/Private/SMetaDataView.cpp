// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMetaDataView.h"

#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SlotBase.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class SWidget;

namespace MetaDataViewColumns
{
	/** IDs for list columns */
	static const FName ColumnID_Tag("Tag");
	static const FName ColumnID_Value("Value");
}

struct FMetaDataLine
{
	FMetaDataLine(FName InTag, const FString& InValue)
		: Tag(InTag)
		, Value(InValue)
	{
	}

	FName Tag;
	FString Value;
};

/**
 * The widget that represents a row in the MetaDataView's list view widget.  Generates a widget for each column, on-demand.
 */
class SMetaDataViewRow : public SMultiColumnTableRow< TSharedPtr< FMetaDataLine > >
{

public:

	SLATE_BEGIN_ARGS(SMetaDataViewRow) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InMetaData			The metadata tag/value to display in the row widget
	 * @param	InOwnerTableView	The owner of the row widget
	 */
	void Construct(const FArguments& InArgs, TSharedRef< FMetaDataLine > InMetaData, TSharedRef< STableViewBase > InOwnerTableView);

	/**
	 * Constructs the widget that represents the specified ColumnID for this Row
	 *
	 * @param ColumnID    A unique ID for a column in this TableView; see SHeaderRow::FColumn for more info.
	 *
	 * @return a widget to represent the contents of a cell in this row of a TableView.
	 */
	virtual TSharedRef< SWidget > GenerateWidgetForColumn(const FName& ColumnID) override;

private:
	TSharedPtr< FMetaDataLine > MetaDataLine;
};

void SMetaDataViewRow::Construct(const FArguments& InArgs, TSharedRef< FMetaDataLine > InMetadata, TSharedRef< STableViewBase > InOwnerTableView)
{
	MetaDataLine = InMetadata;

	SMultiColumnTableRow< TSharedPtr< FMetaDataLine > >::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef< SWidget > SMetaDataViewRow::GenerateWidgetForColumn(const FName& ColumnID)
{
	TSharedPtr< SWidget > TableRowContent;

	static const FTextBlockStyle MetadataTextStyle = FTextBlockStyle(FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		.SetFontSize(10);

	if (ColumnID == MetaDataViewColumns::ColumnID_Tag)
	{
		SAssignNew(TableRowContent, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(1.5f)
		.FillWidth(100.0f)
		[
			SNew(SMultiLineEditableText)
			.Text(FText::FromName(MetaDataLine->Tag))
			.TextStyle(&MetadataTextStyle)
			.IsReadOnly(true)
		];
	}
	else if (ColumnID == MetaDataViewColumns::ColumnID_Value)
	{
		SAssignNew(TableRowContent, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(1.5f)
		.FillWidth(400.0f)
		[
			SNew(SMultiLineEditableText)
			.Text(FText::FromString(*(MetaDataLine->Value)))
			.TextStyle(&MetadataTextStyle)
			.IsReadOnly(true)
			.AutoWrapText(true)
		];
	}
	else
	{
		checkf(false, TEXT("Unknown ColumnID provided to SMetaDataView"));
	}

	return TableRowContent.ToSharedRef();
}

void SMetaDataView::Construct(const FArguments& InArgs, const TMap<FName, FString>& InMetadata)
{
	for (auto It = InMetadata.CreateConstIterator(); It; ++It)
	{
		MetaDataLines.Add(MakeShared<FMetaDataLine>(FMetaDataLine(It->Key, It->Value)));
	}

	TSharedPtr< SHeaderRow > HeaderRowWidget =
		SNew(SHeaderRow)

		// Tag column
		+ SHeaderRow::Column(MetaDataViewColumns::ColumnID_Tag)
		.FillWidth(100.0f)
		.DefaultLabel(NSLOCTEXT("MetadataView", "ColumnID_Tag", "Tag"))
		.DefaultTooltip(FText())

		// Value column
		+ SHeaderRow::Column(MetaDataViewColumns::ColumnID_Value)
		.FillWidth(400.0f)
		.DefaultLabel(NSLOCTEXT("MetadataView", "ColumnID_Value", "Value"))
		.DefaultTooltip(FText());

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			SNew(SListView< TSharedPtr< FMetaDataLine > >)
			.ListItemsSource(&MetaDataLines)
			.OnGenerateRow(this, &SMetaDataView::OnGenerateRow)
			.HeaderRow(HeaderRowWidget)
		]
	];
}

TSharedRef< ITableRow > SMetaDataView::OnGenerateRow(const TSharedPtr< FMetaDataLine > Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	return SNew(SMetaDataViewRow, Item.ToSharedRef(), OwnerTable);
}
