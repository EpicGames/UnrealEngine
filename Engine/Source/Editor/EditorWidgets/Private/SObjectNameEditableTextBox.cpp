// Copyright Epic Games, Inc. All Rights Reserved.

#include "SObjectNameEditableTextBox.h"

#include "ActorEditorUtils.h"
#include "Delegates/Delegate.h"
#include "Framework/Application/SlateApplication.h"
#include "IObjectNameEditSink.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "ObjectNameEditSinkRegistry.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Styling/WidgetStyle.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

class FSlateRect;
struct FSlateBrush;

const float SObjectNameEditableTextBox::HighlightRectLeftOffset = 0.0f;
const float SObjectNameEditableTextBox::HighlightRectRightOffset = 0.0f;
const float SObjectNameEditableTextBox::HighlightTargetSpringConstant = 25.0f;
const float SObjectNameEditableTextBox::HighlightTargetEffectDuration = 0.5f;
const float SObjectNameEditableTextBox::HighlightTargetOpacity = 0.8f;
const float SObjectNameEditableTextBox::CommittingAnimOffsetPercent = 0.2f;

#define LOCTEXT_NAMESPACE "EditorWidgets"

void SObjectNameEditableTextBox::Construct( const FArguments& InArgs )
{
	LastCommittedTime = 0.0;
	bUpdateHighlightSpring = false;

	Objects = InArgs._Objects;
	ObjectNameEditSinkRegistry = InArgs._Registry;


	SAssignNew(TextBox, SInlineEditableTextBlock)
		.Style(FAppStyle::Get(), "DetailsView.NameTextBlockStyle")
		.Text(this, &SObjectNameEditableTextBox::GetNameText)
		.ToolTipText(this, &SObjectNameEditableTextBox::GetNameTooltipText)
		.Visibility(this, &SObjectNameEditableTextBox::GetNameVisibility)
		.OnTextCommitted(this, &SObjectNameEditableTextBox::OnNameTextCommitted)
		.IsReadOnly(this, &SObjectNameEditableTextBox::IsReadOnly)
		.OnVerifyTextChanged_Static(&FActorEditorUtils::ValidateActorName);

	const bool bShouldUseMiddleEllipsis = true; // GetDefault<UEditorStyleSettings>()->bEnableMiddleEllipsis;

	TSharedPtr<STextBlock> SuffixTextBlock = SNew(STextBlock)
		.Text(this, &SObjectNameEditableTextBox::GetObjectDisplayNameSuffix)
		.Visibility(this, &SObjectNameEditableTextBox::GetNameSuffixVisibility)
		.IsEnabled(false);

	ChildSlot
	[	
		SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillContentWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				TextBox.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.FillContentWidth(1000.0f)
			.VAlign(VAlign_Center)
			[
				SuffixTextBlock.ToSharedRef()
			]
	];
}

EActiveTimerReturnType SObjectNameEditableTextBox::UpdateHighlightSpringState( double InCurrentTime, float InDeltaTime )
{
	if ( (float)(InCurrentTime - LastCommittedTime) <= HighlightTargetEffectDuration )
	{
		bUpdateHighlightSpring = true;
	}
	else
	{
		bUpdateHighlightSpring = false;
	}

	return bUpdateHighlightSpring ? EActiveTimerReturnType::Continue : EActiveTimerReturnType::Stop;
}

void SObjectNameEditableTextBox::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	const bool bShouldAppearFocused = HasKeyboardFocus();

	if ( bUpdateHighlightSpring || bShouldAppearFocused )
	{
		// Update highlight 'target' effect
		const float HighlightLeftX = HighlightRectLeftOffset;
		const float HighlightRightX = HighlightRectRightOffset + AllottedGeometry.GetLocalSize().X;

		HighlightTargetLeftSpring.SetTarget( HighlightLeftX );
		HighlightTargetRightSpring.SetTarget( HighlightRightX );

		HighlightTargetLeftSpring.Tick( InDeltaTime );
		HighlightTargetRightSpring.Tick( InDeltaTime );
	}
}

int32 SObjectNameEditableTextBox::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	int32 StartLayer = SCompoundWidget::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );

	const int32 TextLayer = 1;

	// See if a disabled effect should be used
	bool bEnabled = ShouldBeEnabled( bParentEnabled );
	ESlateDrawEffect DrawEffects = (bEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const double CurrentTime = FSlateApplication::Get().GetCurrentTime();

	// Draw highlight targeting effect
	const float TimeSinceHighlightInteraction = (float)( CurrentTime - LastCommittedTime );
	if( TimeSinceHighlightInteraction <= HighlightTargetEffectDuration )
	{
		// Compute animation progress
		float EffectAlpha = FMath::Clamp( TimeSinceHighlightInteraction / HighlightTargetEffectDuration, 0.0f, 1.0f );
		EffectAlpha = 1.0f - EffectAlpha * EffectAlpha;  // Inverse square falloff (looks nicer!)

		float EffectOpacity = EffectAlpha;

		// Figure out a universally visible highlight color.
		FLinearColor HighlightTargetColorAndOpacity = ( (FLinearColor::White - GetColorAndOpacity())*0.5f + FLinearColor(+0.4f, +0.1f, -0.2f)) * InWidgetStyle.GetColorAndOpacityTint();
		HighlightTargetColorAndOpacity.A = HighlightTargetOpacity * EffectOpacity * 255.0f;

		// Compute the bounds offset of the highlight target from where the highlight target spring
		// extents currently lie.  This is used to "grow" or "shrink" the highlight as needed.
		const float CommittingAnimOffset = CommittingAnimOffsetPercent * AllottedGeometry.GetLocalSize().Y;

		// Choose an offset amount depending on whether we're highlighting, or clearing highlight
		const float EffectOffset = EffectAlpha * CommittingAnimOffset;

		const float HighlightLeftX = HighlightTargetLeftSpring.GetPosition() - EffectOffset;
		const float HighlightRightX = HighlightTargetRightSpring.GetPosition() + EffectOffset;
		const float HighlightTopY = 0.0f - EffectOffset;
		const float HighlightBottomY = AllottedGeometry.GetLocalSize().Y + EffectOffset;

		const FVector2D DrawPosition = FVector2D( HighlightLeftX, HighlightTopY );
		const FVector2D DrawSize = FVector2D( HighlightRightX - HighlightLeftX, HighlightBottomY - HighlightTopY );

		const FSlateBrush* StyleInfo = FAppStyle::GetBrush("DetailsView.NameChangeCommitted");

		// NOTE: We rely on scissor clipping for the highlight rectangle
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + TextLayer,
			AllottedGeometry.ToPaintGeometry( DrawSize, FSlateLayoutTransform(DrawPosition) ),	// Position, Size, Scale
			StyleInfo,													// Style
			DrawEffects,												// Effects to use
			HighlightTargetColorAndOpacity );							// Color
	}

	return FMath::Max(StartLayer, LayerId + TextLayer);
}

FText SObjectNameEditableTextBox::GetNameText() const
{
	FText Result;

	if (Objects.Num() == 1)
	{
		Result = GetObjectDisplayName(Objects[0]);
	}
	else if (Objects.Num() > 1)
	{
		if (!UserSetCommonName.IsEmpty())
		{
			Result = FText::FromString(UserSetCommonName);
		}
	}

	return Result;
}

FText SObjectNameEditableTextBox::GetNameTooltipText() const
{
	FText Result;

	if (Objects.Num() == 0)
	{
		Result = LOCTEXT("EditableActorLabel_NoObjectsTooltip", "Nothing selected");
	}
	else if (Objects.Num() == 1)
	{
		UObject* ObjectPtr = Objects[0].Get();
		if (!ObjectPtr)
		{
			return Result;
		}

		TSharedPtr<UE::EditorWidgets::FObjectNameEditSinkRegistry> RegistryPtr = ObjectNameEditSinkRegistry.Pin();
		if (!RegistryPtr.IsValid())
		{
			return Result;
		}

		TSharedPtr<UE::EditorWidgets::IObjectNameEditSink> ObjectNameEditPtr = RegistryPtr->GetObjectNameEditSinkForClass(ObjectPtr->GetClass());
		if (ObjectNameEditPtr)
		{
			Result = ObjectNameEditPtr->GetObjectNameTooltip(ObjectPtr);
		}
	}
	else if (Objects.Num() > 1)
	{
		if (!IsReadOnly())
		{
			Result = LOCTEXT("EditableActorLabel_MultiActorTooltip", "Rename multiple selected actors at once");
		}
		else
		{
			Result = LOCTEXT("EditableActorLabel_NoEditMultiObjectTooltip", "Can't rename selected objects (one or more aren't actors with editable labels)");
		}
	}

	return Result;
}

EVisibility SObjectNameEditableTextBox::GetNameVisibility() const
{
	return ((Objects.Num() == 1 && Objects[0].IsValid()) || Objects.Num() > 1)
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

void SObjectNameEditableTextBox::OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	// Don't apply the change if the TextCommit type is OnCleared - this will only be the case if the keyboard focus was cleared due to
	// Enter being pressed, in which case we will already have been here once with a TextCommit type of OnEnter.
	if (InTextCommit == ETextCommit::OnCleared)
	{
		return;
	}

	FText TrimmedText = FText::TrimPrecedingAndTrailing(NewText);
	if (TrimmedText.IsEmpty())
	{
		return;
	}

	TSharedPtr<UE::EditorWidgets::FObjectNameEditSinkRegistry> RegistryPtr = ObjectNameEditSinkRegistry.Pin();
	if (!RegistryPtr.IsValid())
	{
		return;
	}

	UserSetCommonName = TrimmedText.ToString();

	const FScopedTransaction Transaction( LOCTEXT("RenameActorsTransaction", "Rename Multiple Actors") );

	bool bChanged = false;
	for (TWeakObjectPtr<UObject> Object : Objects)
	{
		UObject* ObjectPtr = Object.Get();
		if (!ObjectPtr)
		{
			continue;
		}

		TSharedPtr<UE::EditorWidgets::IObjectNameEditSink> ObjectNameEditPtr = RegistryPtr->GetObjectNameEditSinkForClass(ObjectPtr->GetClass());
		if (ObjectNameEditPtr && ObjectNameEditPtr->SetObjectDisplayName(ObjectPtr, UserSetCommonName))
		{
			bChanged = true;
		}
	}

	if (bChanged)
	{
		LastCommittedTime = FSlateApplication::Get().GetCurrentTime();
		RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SObjectNameEditableTextBox::UpdateHighlightSpringState ) );
	}

	// Remove ourselves from the window focus so we don't get automatically reselected when scrolling around the context menu.
	TSharedPtr< SWindow > ParentWindow = FSlateApplication::Get().FindWidgetWindow( SharedThis(this) );
	if (ParentWindow.IsValid())
	{
		ParentWindow->SetWidgetToFocusOnActivate(nullptr);
	}
}

bool SObjectNameEditableTextBox::IsReadOnly() const
{
	if (Objects.Num() == 0)
	{
		// can't edit if nothing is selected
		return true;
	}

	TSharedPtr<UE::EditorWidgets::FObjectNameEditSinkRegistry> RegistryPtr = ObjectNameEditSinkRegistry.Pin();
	// this shouldn't happen but let's be safe and say readonly
	if (!RegistryPtr.IsValid())
	{
		return true;
	}

	for (TWeakObjectPtr<UObject> Object : Objects)
	{
		UObject* ObjectPtr = Object.Get();
		if (!ObjectPtr)
		{
			continue;
		}

		TSharedPtr<UE::EditorWidgets::IObjectNameEditSink> ObjectNameEditPtr = RegistryPtr->GetObjectNameEditSinkForClass(ObjectPtr->GetClass());
		if (ObjectNameEditPtr && ObjectNameEditPtr->IsObjectDisplayNameReadOnly(ObjectPtr))
		{
			// if any objects are read only then the whole box is
			return true;
		}
	}
	return false;
}

FText SObjectNameEditableTextBox::GetObjectDisplayName(TWeakObjectPtr<UObject> Object) const
{
	UObject* ObjectPtr = Object.Get();
	// no display name for no object
	if (!ObjectPtr)
	{
		return FText();
	}
	TSharedPtr<UE::EditorWidgets::FObjectNameEditSinkRegistry> RegistryPtr = ObjectNameEditSinkRegistry.Pin();
	if (RegistryPtr.IsValid())
	{
		TSharedPtr<UE::EditorWidgets::IObjectNameEditSink> ObjectNameEditPtr = RegistryPtr->GetObjectNameEditSinkForClass(ObjectPtr->GetClass());
		if (ObjectNameEditPtr)
		{
			return ObjectNameEditPtr->GetObjectDisplayName(ObjectPtr);
		}
	}
	// this shouldn't happen but let's be safe and just give the default name
	return FText::FromString(ObjectPtr->GetName());
}

EVisibility SObjectNameEditableTextBox::GetNameSuffixVisibility() const
{
	if (const TSharedPtr<const UE::EditorWidgets::FObjectNameEditSinkRegistry> RegistryPtr = ObjectNameEditSinkRegistry.Pin())
	{
		if (Objects.Num() == 1)
		{
			if (const UObject* const ObjectPtr = Objects[0].Get())
			{
				const TSharedPtr<const UE::EditorWidgets::IObjectNameEditSink> ObjectNameEditPtr = RegistryPtr->GetObjectNameEditSinkForClass(ObjectPtr->GetClass());
				if (ObjectNameEditPtr)
				{
					return ObjectNameEditPtr->ObjectDisplayNameHasSuffix(ObjectPtr) ? EVisibility::Visible : EVisibility::Collapsed;
				}
			}
		}
	}

	return EVisibility::Collapsed;
}

FText SObjectNameEditableTextBox::GetObjectDisplayNameSuffix() const
{
	if (const TSharedPtr<const UE::EditorWidgets::FObjectNameEditSinkRegistry> RegistryPtr = ObjectNameEditSinkRegistry.Pin())
	{
		if (Objects.Num() == 1)
		{
			if (const UObject* const ObjectPtr = Objects[0].Get())
			{
				const TSharedPtr<const UE::EditorWidgets::IObjectNameEditSink> ObjectNameEditPtr = RegistryPtr->GetObjectNameEditSinkForClass(ObjectPtr->GetClass());
				if (ObjectNameEditPtr)
				{
					return ObjectNameEditPtr->GetObjectDisplayNameSuffix(ObjectPtr);
				}
			}
		}
	}

	return FText();
}


// No code beyond this point
#undef LOCTEXT_NAMESPACE
