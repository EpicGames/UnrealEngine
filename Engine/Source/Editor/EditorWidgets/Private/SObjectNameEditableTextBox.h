// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/SlateSprings.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EditorWidgetsModule.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
class SInlineEditableTextBlock;
class UObject;
namespace UE::EditorWidgets { class FObjectNameEditSinkRegistry; }
struct FGeometry;

/** Widget wraps an editable text box for viewing the names of objects or editing the labels of actors */
class SObjectNameEditableTextBox : public IObjectNameEditableTextBox
{
public:
	SLATE_BEGIN_ARGS(SObjectNameEditableTextBox){}

		SLATE_ARGUMENT(TArray<TWeakObjectPtr<UObject>>, Objects)

		SLATE_ARGUMENT(TWeakPtr<UE::EditorWidgets::FObjectNameEditSinkRegistry>, Registry)

	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct( const FArguments& InArgs );

protected:

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;

private:
	/** Updates whether the highlight spring effect is happening */
	EActiveTimerReturnType UpdateHighlightSpringState( double InCurrentTime, float InDeltaTime );

	/** Getter for the Text attribute of the editable text inside this widget */
	FText GetNameText() const;

	/** Should the name editing text box even be visible? */
	EVisibility GetNameVisibility() const;

	/** Getter for the ToolTipText attribute of the editable text inside this widget */
	FText GetNameTooltipText() const;

	/** Getter for the OnTextCommitted event of the editable text inside this widget */
	void OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);

	/** Getter for the IsReadOnly attribute of the editable text inside this widget */
	bool IsReadOnly() const;

	/** Helper function to access name registry and retrieve display name */
	FText GetObjectDisplayName(TWeakObjectPtr<UObject> Object) const;

	/** Helper function to access name registry and determine if the name has a suffix */
	EVisibility GetNameSuffixVisibility() const;

	/** Access name registry and retrieve the name suffix, if there is one */
	FText GetObjectDisplayNameSuffix() const;

	/** The list of objects whose names are edited by the widget */
	TArray<TWeakObjectPtr<UObject>> Objects;

	/** The current user-entered text for a list of more than one object */
	FString UserSetCommonName;

	/** How many pixels to extend the highlight rectangle's left side horizontally */
	static const float HighlightRectLeftOffset;

	/** How many pixels to extend the highlight rectangle's right side horizontally */
	static const float HighlightRectRightOffset;

	/** How quickly the highlight 'targeting' rectangle will slide around.  Larger is faster. */
	static const float HighlightTargetSpringConstant;

	/** Duration of animation highlight target effects */
	static const float HighlightTargetEffectDuration;

	/** Opacity of the highlight target effect overlay */
	static const float HighlightTargetOpacity;

	/** How large the highlight target effect will be when highlighting, as a scalar percentage of font height */
	static const float CommittingAnimOffsetPercent;

	/** Highlight "targeting" visual effect left position */
	FFloatSpring1D HighlightTargetLeftSpring;

	/** Highlight "targeting" visual effect right position */
	FFloatSpring1D HighlightTargetRightSpring;

	/** Last time that the user had a major interaction with the highlight */
	double LastCommittedTime;

	/** The text box used to edit object names */ 
	TSharedPtr< SInlineEditableTextBlock > TextBox;

	/** The registry for object names */
	TWeakPtr<UE::EditorWidgets::FObjectNameEditSinkRegistry> ObjectNameEditSinkRegistry;

	// Temp flag to trigger a highlight spring update in the passive tick (because that's where the geometry is)
	bool bUpdateHighlightSpring;
};
