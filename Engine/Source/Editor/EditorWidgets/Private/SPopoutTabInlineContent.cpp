// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPopoutTabInlineContent.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "InputCoreTypes.h"
#include "Interfaces/IMainFrameModule.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "SPopoutTabInlineContent"

SPopoutTabInlineContent::~SPopoutTabInlineContent()
{
	UnsubscribeFromTabForegrounded();

	UnregisterAutoSizeTimer();

	// Avoid complex tab operations during engine shutdown
	if (!IsEngineExitRequested() && FSlateApplication::IsInitialized())
	{
		// Request the tab to close. OnTabClosed will restore safely if it actually closes.
		CloseTab();
		UnregisterTab();

		// If there was no tab or we're in teardown and it won't close, this will restore if detached
		if (!WeakSpawnedDockTab.IsValid())
		{
			RestoreInlineContent();
		}
	}
}

void SPopoutTabInlineContent::Construct(const FArguments& InArgs)
{
	WeakContentWidget = InArgs._Content.Widget;
	WeakTabManager = InArgs._TabManager.IsValid() ? InArgs._TabManager.Pin().ToSharedRef() : FGlobalTabmanager::Get();
	TabId = InArgs._TabId;
	TabDisplayName = InArgs._TabDisplayName;
	TabIcon = InArgs._TabIcon;
	TabRole = InArgs._TabRole;
	WeakTabGroup = InArgs._TabGroup;
	bAutosize = InArgs._Autosize;
	bAutosizeWhenFloating = InArgs._AutosizeWhenFloating;
	bCanDragToPopout = InArgs._CanDragToPopout;
	bShowDragBorder = InArgs._ShowDragBorder;
	bHideDockTabStackTab = InArgs._HideDockTabStackTab;
	PopoutStateChangedDelegate = InArgs._OnPopoutStateChanged;
	GetMenuContentDelegate = InArgs._OnGetMenuContent;

	static const FSlateRoundedBoxBrush BorderHandleBrush(FLinearColor::Transparent
		, 0.f, FStyleColors::AccentBlue, 2.f);

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Visibility(this, &SPopoutTabInlineContent::GetContentVisibility)
		[
			SNew(SBox)
			.HAlign(InArgs._ContentHAlign)
			.VAlign(InArgs._ContentVAlign)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SAssignNew(InlineContentContainer, SBox)
					[
						InArgs._Content.Widget
					]
				]
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(&BorderHandleBrush)
					.Visibility(this, &SPopoutTabInlineContent::GetContentBorderVisibility)
				]
			]
		]
	];

	// If we previously detached content but couldn't restore (e.g. container was invalid at close time),
	// try again now that InlineContentContainer exists
	RestoreInlineContent();

	if (InArgs._InitiallyPoppedOut)
	{
		Popout(true);

		RegisterAutoSizeTimer();
	}
}

TSharedRef<SButton> SPopoutTabInlineContent::CreateToggleButton(const TSharedRef<SPopoutTabInlineContent>& InOwner
	, const FText& InTooltipText
	, const FSlateBrush* const InIconBrush)
{
	constexpr float ToolButtonImageSize = 12.f;

	const TWeakPtr<SPopoutTabInlineContent> WeakOwner = InOwner;

	return SNew(SButton)
		.VAlign(VAlign_Center)
		.ToolTipText(InTooltipText)
		.Visibility_Lambda([WeakOwner]()
			{
				if (const TSharedPtr<SPopoutTabInlineContent> Owner = WeakOwner.Pin())
				{
					return Owner->IsPoppedOut() ? EVisibility::Collapsed : EVisibility::Visible;
				}
				return EVisibility::Collapsed;
			})
		.OnClicked_Lambda([WeakOwner]()
			{
				if (const TSharedPtr<SPopoutTabInlineContent> Owner = WeakOwner.Pin())
				{
					Owner->Popout(true);
				}
				return FReply::Handled();
			})
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2D(ToolButtonImageSize, ToolButtonImageSize))
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(InIconBrush)
		];
}

bool SPopoutTabInlineContent::IsPoppedOut() const
{
	return bPoppedOut;
}

void SPopoutTabInlineContent::Popout(const bool bInPopout)
{
	if (bInPopout)
	{
		RegisterTab();

		if (bTabRegistered)
		{
			if (const TSharedPtr<FTabManager> TabManager = GetTabManager())
			{
				if (const TSharedPtr<SDockTab> Tab = TabManager->TryInvokeTab(TabId))
				{
					BroadcastPopoutStateChanged(true);
				}
				else
				{
					UnregisterTab();
					BroadcastPopoutStateChanged(false);
				}
			}
		}
	}
	else
	{
		CloseTab();
	}

	bLeftMouseDownOverWidget = false;
}

void SPopoutTabInlineContent::TogglePopout()
{
	Popout(!IsPoppedOut());
}

TSharedPtr<FTabManager> SPopoutTabInlineContent::GetTabManager() const
{
	return WeakTabManager.Pin();
}

bool SPopoutTabInlineContent::IsUniqueTabId() const
{
	if (TabId.IsNone())
	{
		return false;
	}

	const TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (!TabManager.IsValid())
	{
		return false;
	}

	// If anyone already registered a spawner under this id, we might hijack their tab
	if (TabManager->HasTabSpawner(TabId))
	{
		return false;
	}

	// If a live tab already exists under this id, we might be pointing at someone else’s instance
	if (TabManager->FindExistingLiveTab(TabId).IsValid())
	{
		return false;
	}

	return true;
}

void SPopoutTabInlineContent::RegisterTab()
{
	if (bTabRegistered)
	{
		return;
	}

	if (!IsUniqueTabId())
	{
		UE_LOGF(LogTemp, Warning
			, "SPopoutTabInlineContent: TabId '%ls' conflicts with an existing spawner or live tab; refusing to register."
			, *TabId.ToString());
		return;
	}

	const TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (!TabManager.IsValid())
	{
		return;
	}

	FTabSpawnerEntry& TabSpawnerEntry = TabManager->RegisterTabSpawner(TabId
		, FOnSpawnTab::CreateSP(this, &SPopoutTabInlineContent::SpawnTab))
			.SetDisplayName(TabDisplayName)
			.SetIcon(TabIcon);

	if (const TSharedPtr<FWorkspaceItem> TabGroup = WeakTabGroup.Pin())
	{
		TabSpawnerEntry.SetGroup(TabGroup.ToSharedRef());
	}

	bTabRegistered = true;
}

void SPopoutTabInlineContent::UnregisterTab()
{
	if (!bTabRegistered)
	{
		return;
	}

	if (const TSharedPtr<FTabManager> TabManager = GetTabManager())
	{
		TabManager->UnregisterTabSpawner(TabId);
	}

	bTabRegistered = false;
}

TSharedRef<SDockTab> SPopoutTabInlineContent::SpawnTab(const FSpawnTabArgs& InArgs)
{
	const TSharedPtr<SWidget> ContentWidget = WeakContentWidget.IsValid()
		? WeakContentWidget.Pin() : DetachedContentWidget;

	// If we have the real widget, remove it from the inline container before reparenting
	if (ContentWidget.IsValid() && InlineContentContainer.IsValid())
	{
		DetachedContentWidget = ContentWidget;
		InlineContentContainer->SetContent(SNullWidget::NullWidget);
	}

	const TSharedRef<SWidget> ActualContentWidget = DetachedContentWidget.IsValid()
		? DetachedContentWidget.ToSharedRef() : SNullWidget::NullWidget;

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(TabRole)
		.ShouldAutosize(bAutosize)
		.OnTabActivated(this, &SPopoutTabInlineContent::OnTabActivated)
		.OnTabClosed(this, &SPopoutTabInlineContent::OnTabClosed)
		.OnTabRelocated(this, &SPopoutTabInlineContent::OnTabRelocated)
		[
			ActualContentWidget
		];
	WeakSpawnedDockTab = DockTab;

	RegisterAutoSizeTimer(DockTab);

	SubscribeToTabForegrounded();

	BroadcastPopoutStateChanged(true);

	return DockTab;
}

void SPopoutTabInlineContent::CloseTab()
{
	if (const TSharedPtr<SDockTab> ExistingTab = WeakSpawnedDockTab.Pin())
	{
		// Do NOT restore here. The tab still owns the widget.
		// OnTabClosed will call RestoreInlineContent() safely.
		ExistingTab->RequestCloseTab();

		return;
	}

	// If we don't have a spawned tab reference, we can't safely assume a tab isn't open.
	// Best-effort: restore only if we are detached and there is no known live tab mechanism.
	RestoreInlineContent();

	BroadcastPopoutStateChanged(false);
}

void SPopoutTabInlineContent::OnTabActivated(const TSharedRef<SDockTab> InDockTab, const ETabActivationCause InActivationCause)
{
	RegisterAutoSizeTimer(InDockTab);

	BroadcastPopoutStateChanged(true);
}

void SPopoutTabInlineContent::OnTabClosed(const TSharedRef<SDockTab> InDockTab)
{
	UnregisterAutoSizeTimer();

	// Restore only once the tab is actually closed so the widget isn't double-parented
	WeakSpawnedDockTab.Reset(); // reset first so RestoreInlineContent won't bail
	RestoreInlineContent();

	UnregisterTab();

	UnsubscribeFromTabForegrounded();

	bIsForeground = false;

	BroadcastPopoutStateChanged(false);
}

void SPopoutTabInlineContent::OnTabRelocated()
{
	TryHideStackTab();

	RegisterAutoSizeTimer();
}

void SPopoutTabInlineContent::OnTabForegrounded(const TSharedPtr<SDockTab> InNewTab, const TSharedPtr<SDockTab> InPreviousTab)
{
	if (!InNewTab.IsValid())
	{
		return;
	}

	const TSharedPtr<SDockTab> SpawnedTab = WeakSpawnedDockTab.Pin();

	const auto IsOurTab = [&SpawnedTab](const TSharedPtr<SDockTab>& Tab) -> bool
		{
			return Tab.IsValid() && (Tab == SpawnedTab);
		};

	const bool bNewIsOurs = IsOurTab(InNewTab);
	const bool bPrevIsOurs = IsOurTab(InPreviousTab);

	if (!bNewIsOurs)
	{
		bIsForeground = false;
		return;
	}

	const bool bBecameForeground = !bPrevIsOurs || !bIsForeground;
	bIsForeground = true;

	if (bBecameForeground)
	{
		TryHideStackTab();
	}
}

FCursorReply SPopoutTabInlineContent::OnCursorQuery(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) const
{
	if (bLeftMouseDownOverWidget && bCanDragToPopout)
	{
		return FCursorReply::Cursor(EMouseCursor::GrabHand);
	}
	return FCursorReply::Unhandled();
}

FReply SPopoutTabInlineContent::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	const FKey EffectingButton = InPointerEvent.GetEffectingButton();
	const bool bLeftMouseDown = EffectingButton == EKeys::LeftMouseButton;
	const bool bRightMouseDown = EffectingButton == EKeys::RightMouseButton;

	if (bLeftMouseDown)
	{
		if (bCanDragToPopout && !IsPoppedOut())
		{
			bLeftMouseDownOverWidget = true;

			// Keep receiving move events even if the cursor leaves the widget while dragging
			return FReply::Handled()
				.CaptureMouse(SharedThis(this))
				.DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
		}
	}
	else if (bRightMouseDown)
	{
		bRightMouseDownOverWidget = true;
	}

	return FReply::Unhandled();
}

FReply SPopoutTabInlineContent::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	const FKey EffectingButton = InPointerEvent.GetEffectingButton();
	const bool bLeftMouseUp = EffectingButton == EKeys::LeftMouseButton;
	const bool bRightMouseUp = EffectingButton == EKeys::RightMouseButton;

	if (bLeftMouseUp)
	{
		bLeftMouseDownOverWidget = false;

		// Release capture before popping out
		FReply Reply = HasMouseCapture() ? FReply::Handled().ReleaseMouseCapture() : FReply::Unhandled();

		if (bWantsToPopout)
		{
			bWantsToPopout = false;
			Popout(true);
		}

		return Reply;
	}

	if (bRightMouseUp)
	{
		if (bRightMouseDownOverWidget)
		{
			bRightMouseDownOverWidget = false;

			// Show the context menu
			const TSharedRef<SWidget> ParentWidget = SharedThis(this);
			const FWidgetPath* const EventPath = InPointerEvent.GetEventPath();
			const FWidgetPath WidgetPath = EventPath ? *EventPath : FWidgetPath();
			const TSharedRef<SWidget> MenuContentWidget = GetMenuContentDelegate.IsBound()
				? GetMenuContentDelegate.Execute() : CreateDefaultMenuContent();

			FSlateApplication::Get().PushMenu(ParentWidget
				, WidgetPath
				, MenuContentWidget
				, InPointerEvent.GetScreenSpacePosition()
				, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void SPopoutTabInlineContent::OnMouseLeave(const FPointerEvent& InPointerEvent)
{
	// Don't clear drag state while the left button is held. Leaving the widget during a drag is normal.
	if (!InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		bLeftMouseDownOverWidget = false;
	}

	bRightMouseDownOverWidget = false;

	bWantsToPopout = false;
}

FReply SPopoutTabInlineContent::OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton)
		&& bCanDragToPopout
		&& !IsPoppedOut())
	{
		bWantsToPopout = true;
	}

	bLeftMouseDownOverWidget = false;

	// We’re not starting a drag-drop operation, just using drag as a gesture
	return FReply::Handled().ReleaseMouseCapture();
}

TSharedRef<SWidget> SPopoutTabInlineContent::CreateDefaultMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection(TEXT("ContentOptions"), LOCTEXT("ContentOptions", "Options"));

	MenuBuilder.AddMenuEntry(LOCTEXT("PopoutContent", "Popout")
		, LOCTEXT("PopoutContentTooltip", "Popout this content to its own tab window")
		, FSlateIcon()
		, FExecuteAction::CreateSP(this, &SPopoutTabInlineContent::Popout, true)
	);

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

EVisibility SPopoutTabInlineContent::GetContentVisibility() const
{
	return IsPoppedOut() ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SPopoutTabInlineContent::GetContentBorderVisibility() const
{
	return (bShowDragBorder && bLeftMouseDownOverWidget && bCanDragToPopout)
		? EVisibility::Visible : EVisibility::Hidden;
}

void SPopoutTabInlineContent::RegisterAutoSizeTimer(const TSharedPtr<SDockTab>& InDockTab)
{
	// This function and the timer it creates is only used for auto-sizing when popped out
	if (!bAutosize && !bAutosizeWhenFloating)
	{
		return;
	}

	if (SetupWindowTimerHandle.IsValid())
	{
		return;
	}

	const TSharedPtr<SDockTab> ExistingTab = InDockTab.IsValid() ? InDockTab : WeakSpawnedDockTab.Pin();
	if (!ExistingTab.IsValid())
	{
		return;
	}

	const TWeakPtr<SPopoutTabInlineContent> ThisWeak = SharedThis(this);
	const TWeakPtr<SDockTab> WeakTab = ExistingTab;

	/** Need to use ExistingTab to register the timer since this widget can be hidden */
	SetupWindowTimerHandle = ExistingTab->RegisterActiveTimer(0.f
		, FWidgetActiveTimerDelegate::CreateLambda(
			[ThisWeak, WeakTab](const double InCurrentTime, const float InDeltaTime) -> EActiveTimerReturnType
			{
				if (const TSharedPtr<SPopoutTabInlineContent> This = ThisWeak.Pin())
				{
					return This->HandleAutoSizeWindowTimer(InCurrentTime, InDeltaTime, WeakTab);
				}
				return EActiveTimerReturnType::Stop;
			}));
}

void SPopoutTabInlineContent::UnregisterAutoSizeTimer()
{
	if (!SetupWindowTimerHandle.IsValid())
	{
		return;
	}

	if (const TSharedPtr<SDockTab> ExistingTab = WeakSpawnedDockTab.Pin())
	{
		ExistingTab->UnRegisterActiveTimer(SetupWindowTimerHandle.ToSharedRef());
	}

	SetupWindowTimerHandle.Reset();
}

EActiveTimerReturnType SPopoutTabInlineContent::HandleAutoSizeWindowTimer(const double InCurrentTime
	, const float InDeltaTime, const TWeakPtr<SDockTab> InWeakTab)
{
	const TSharedPtr<SDockTab> ExistingTab = InWeakTab.Pin();
	if (!ExistingTab.IsValid())
	{
		SetupWindowTimerHandle.Reset();
		return EActiveTimerReturnType::Stop;
	}

	// Wait until the window is actually constructed/visible, otherwise sizing/locking
	// can be ignored or overwritten by later docking/window setup.
	const TSharedPtr<SWindow> Window = ExistingTab->GetParentWindow();
	if (!Window.IsValid() || !Window->GetNativeWindow().IsValid())
	{
		// Parent window not assigned yet (common on first show). Keep waiting.
		return EActiveTimerReturnType::Continue;
	}

	const bool bIsVisible = Window->IsVisible();
	const FVector2D SizeInScreen = Window->GetSizeInScreen();
	const bool bHasNonZeroSize = SizeInScreen.X > 1.f && SizeInScreen.Y > 1.f;
	if (!bIsVisible || !bHasNonZeroSize)
	{
		return EActiveTimerReturnType::Continue;
	}

	const bool bShouldAutosizeFloating = bAutosizeWhenFloating && IsTabFloating(ExistingTab.ToSharedRef());
	if (!bAutosize && !bShouldAutosizeFloating)
	{
		SetupWindowTimerHandle.Reset();
		return EActiveTimerReturnType::Stop;
	}

	// An auto-sized tab only sets the size of the tab to the contents when the tab is first displayed.
	// Switch to a fixed layout after we use the auto-sized content to get the desired size.
	// Temporarily enable auto sizing and force a layout pass to measure the widget tree's final
	// desired size, then use this to set the size of the window based on the content
	Window->SetSizingRule(ESizingRule::Autosized);
	Window->SlatePrepass(GetPrepassLayoutScaleMultiplier());
	Window->Resize(Window->GetDesiredSize());
	// FixedSize prevents the user from dragging window edges to resize
	Window->SetSizingRule(ESizingRule::FixedSize);

	SetupWindowTimerHandle.Reset();

	return EActiveTimerReturnType::Stop;
}

void SPopoutTabInlineContent::TryHideStackTab()
{
	if (!bHideDockTabStackTab)
	{
		return;
	}

	const TSharedPtr<SDockTab> ExistingTab = WeakSpawnedDockTab.Pin();
	if (!ExistingTab.IsValid())
	{
		return;
	}

	ExistingTab->SetParentDockTabStackTabWellHidden(true);
}

void SPopoutTabInlineContent::BroadcastPopoutStateChanged(const bool bInNewPoppedOut)
{
	if (bPoppedOut != bInNewPoppedOut)
	{
		bPoppedOut = bInNewPoppedOut;
		PopoutStateChangedDelegate.ExecuteIfBound(bPoppedOut);
	}
}

void SPopoutTabInlineContent::RestoreInlineContent()
{
	if (!DetachedContentWidget.IsValid())
	{
		return;
	}

	// Don't reparent while our spawned tab is still alive. The tab likely still owns the widget.
	if (WeakSpawnedDockTab.IsValid())
	{
		return;
	}

	// Restore content back into the inline container so it has exactly one parent again.
	// We only ever set InlineContentContainer content from this class.
	// DetachedContentWidget being valid is our "detached and must restore" signal.
	if (InlineContentContainer.IsValid())
	{
		const TSharedRef<SWidget> WidgetToRestore = DetachedContentWidget.ToSharedRef();
		DetachedContentWidget.Reset();

		InlineContentContainer->SetContent(WidgetToRestore);

		return;
	}

	ensureMsgf(false, TEXT("RestoreInlineContent: InlineContentContainer invalid while DetachedContentWidget is valid."));

	// Reset during teardown to avoid holding UI alive / leaking refs
	if (!FSlateApplication::IsInitialized())
	{
		DetachedContentWidget.Reset();
	}
}

void SPopoutTabInlineContent::SubscribeToTabForegrounded()
{
	if (IsEngineExitRequested() || !FSlateApplication::IsInitialized())
	{
		return;
	}

	if (TabForegroundedDelegateHandle.IsValid())
	{
		return;
	}

	TabForegroundedDelegate = FOnActiveTabChanged::FDelegate::CreateSP(this, &SPopoutTabInlineContent::OnTabForegrounded);
	TabForegroundedDelegateHandle = FGlobalTabmanager::Get()->OnTabForegrounded_Subscribe(TabForegroundedDelegate);
}

void SPopoutTabInlineContent::UnsubscribeFromTabForegrounded()
{
	if (TabForegroundedDelegateHandle.IsValid() && FSlateApplication::IsInitialized() && !IsEngineExitRequested())
	{
		FGlobalTabmanager::Get()->OnTabForegrounded_Unsubscribe(TabForegroundedDelegateHandle);
	}

	TabForegroundedDelegate.Unbind();
	TabForegroundedDelegateHandle.Reset();
}

bool SPopoutTabInlineContent::IsTabFloating(const TSharedRef<SDockTab>& InDockTab)
{
	if (!FSlateApplication::IsInitialized())
	{
		return false;
	}

	static const FName MainFrameModuleName = TEXT("MainFrame");
	IMainFrameModule* const MainFrame = FModuleManager::GetModulePtr<IMainFrameModule>(MainFrameModuleName);
	if (!MainFrame)
	{
		return false;
	}

	const TSharedPtr<SWindow> MainParentWindow = MainFrame->GetParentWindow();
	const TSharedPtr<SWindow> TabParentWindow = FSlateApplication::Get().FindWidgetWindow(InDockTab);

	const bool bFloating = MainParentWindow.IsValid() && TabParentWindow.IsValid() && (MainParentWindow != TabParentWindow);
	return bFloating;
}

#undef LOCTEXT_NAMESPACE
