// Copyright Epic Games, Inc. All Rights Reserved.

#include "STemplateStringEditableTextBox.h"

#include "EditorWidgetsStyle.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Regex.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "SlateOptMacros.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "TemplateStringSyntaxHighlighterMarshaller.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SPopUpErrorText.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "STemplateStringEditableTextBox"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void STemplateStringEditableTextBox::Construct(const FArguments& InArgs)
{
	if (InArgs._Style == nullptr)
	{
		SetStyle(&FEditorWidgetsStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"));		
	}
	else
	{
		SetStyle(InArgs._Style);
	}

	static const FVector2D ScrollBarThickness(9.0f, 9.0f);
	static const FMargin ErrorWidgetPadding(3, 0); 

	TokenizedText = InArgs._Text;
	ResolvedText = InArgs._ResolvedText;
	ValidArguments = InArgs._ValidArguments;

	OnValidateTokenizedText = InArgs._OnValidateTokenizedText;
	OnTokenizedTextChanged = InArgs._OnTextChanged;
	OnTokenizedTextCommitted = InArgs._OnTextCommitted;

	ErrorReporting = SNew(SPopupErrorText);
	ErrorReporting->SetError(FText::GetEmpty());

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(this, &STemplateStringEditableTextBox::GetBorderImage)
		.BorderBackgroundColor(TextBoxStyle->BackgroundColor)
		.ForegroundColor(this, &SMultiLineEditableTextBox::GetForegroundColor)
		.Padding(TextBoxStyle->Padding)
		[
			SAssignNew(Box, SHorizontalBox)

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.AutoWidth()
			.Padding(4, 0)
			[
				SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.TokenTextBox"))
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillWidth(1.0f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.FillHeight(1.0f)
				[
					SAssignNew(EditableText, SMultiLineEditableText)
					.Text(this, &STemplateStringEditableTextBox::GetText)
					.TextStyle(&TextBoxStyle->TextStyle)
					.Marshaller(FTemplateStringSyntaxHighlighterMarshaller::Create(FTemplateStringSyntaxHighlighterMarshaller::FSyntaxTextStyle()))
					.AllowMultiLine(InArgs._AllowMultiLine)
					.IsReadOnly(InArgs._IsReadOnly)
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					.Margin(0.0f)
					.OnTextChanged(this, &STemplateStringEditableTextBox::OnEditableTextChanged)
					.OnTextCommitted(this, &STemplateStringEditableTextBox::OnEditableTextCommitted)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(HScrollBarBox, SBox)
					.Padding(TextBoxStyle->HScrollBarPadding)
					[
						SAssignNew(HScrollBar, SScrollBar)
						.Style(&TextBoxStyle->ScrollBarStyle)
						.Orientation(Orient_Horizontal)
						.Thickness(ScrollBarThickness)
					]
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(VScrollBarBox, SBox)
				.Padding(TextBoxStyle->VScrollBarPadding)
				[
					SAssignNew(VScrollBar, SScrollBar)
					.Style(&TextBoxStyle->ScrollBarStyle)
					.Orientation(Orient_Vertical)
					.Thickness(ScrollBarThickness)
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(ErrorWidgetPadding)
			[
				ErrorReporting->AsWidget()
			]
		]
	];
}

FText STemplateStringEditableTextBox::GetText() const
{
	if (HasKeyboardFocus())
	{
		return TokenizedText.Get();
	}
	else
	{
		if (ResolvedText.IsSet())
		{
			return ResolvedText.Get();
		}
		else
		{
			return TokenizedText.Get();
		}
	}
}

const FSlateBrush* STemplateStringEditableTextBox::GetBorderImage() const
{
	check(TextBoxStyle);
	check(EditableText.IsValid());

	if (EditableText->IsTextReadOnly())
	{
		return &TextBoxStyle->BackgroundImageReadOnly;
	}
	else if (EditableText->HasKeyboardFocus())
	{
		return &TextBoxStyle->BackgroundImageFocused;
	}
	else if (EditableText->IsHovered())
	{
		return &TextBoxStyle->BackgroundImageHovered;
	}
	else
	{
		return &TextBoxStyle->BackgroundImageNormal;
	}
}

FSlateColor STemplateStringEditableTextBox::GetForegroundColor() const
{
	check(TextBoxStyle);

	return HasKeyboardFocus()
		? TextBoxStyle->FocusedForegroundColor
		: TextBoxStyle->ForegroundColor;
}

void STemplateStringEditableTextBox::SetStyle(const FEditableTextBoxStyle* InStyle)
{
	if (InStyle)
	{
		TextBoxStyle = InStyle;
	}
	else
	{
		FArguments Defaults;
		TextBoxStyle = Defaults._Style;
	}

	check(TextBoxStyle);
	
	if (EditableText.IsValid())
	{
		EditableText->SetTextStyle(&TextBoxStyle->TextStyle);
	}
}

void STemplateStringEditableTextBox::SetError(const FText& InError) const
{
	ErrorReporting->SetError(InError);
}

bool STemplateStringEditableTextBox::HasKeyboardFocus() const
{
	// Since keyboard focus is forwarded to our editable text, we will test it instead
	return SCompoundWidget::HasKeyboardFocus() || EditableText->HasKeyboardFocus();
}

FReply STemplateStringEditableTextBox::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	FReply Reply = FReply::Handled();
	if ( InFocusEvent.GetCause() != EFocusCause::Cleared )
	{
		// Forward keyboard focus to our editable text widget
		Reply.SetUserFocus(EditableText.ToSharedRef(), InFocusEvent.GetCause());
	}

	return Reply;
}

FReply STemplateStringEditableTextBox::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const FKey Key = InKeyEvent.GetKey();
	if (Key == EKeys::Escape && EditableText->HasKeyboardFocus())
	{
		// Clear focus
		return FReply::Handled().SetUserFocus(SharedThis(this), EFocusCause::Cleared);
	}

	return FReply::Unhandled();
}

bool STemplateStringEditableTextBox::ValidateTokenizedText(const FText& InTokenizedText)
{
	FTextBuilder ErrorMessageBuilder;
	FText ErrorMessage;

	if (!ValidateTokenBraces(InTokenizedText, ErrorMessage))
	{
		ErrorMessageBuilder.AppendLine(ErrorMessage);
	}

	// Only validate tokens if there were no other errors - it's expensive
	if (ErrorMessage.IsEmpty() && !ValidateTokenArgs(InTokenizedText, ErrorMessage))
	{
		ErrorMessageBuilder.AppendLine(ErrorMessage);
	}

	if (!ErrorMessageBuilder.IsEmpty())
	{
		ErrorMessage = ErrorMessageBuilder.ToText();
	}

	if (ErrorMessage.IsEmpty()	&& OnValidateTokenizedText.IsBound())
	{
		OnValidateTokenizedText.Execute(InTokenizedText, ErrorMessage);
	}

	SetError(ErrorMessage);

	// If the error message is empty, the tokenized text is valid
	return ErrorMessage.IsEmpty();
}

bool STemplateStringEditableTextBox::ValidateTokenBraces(const FText& InTokenizedText, FText& OutError)
{
	// Get number of open and close braces in string
	int32 OpenBraceCount = 0;
	int32 CloseBraceCount = 0;

	bool bLastBraceWasOpen = false;

	auto GetUnbalancedBracesMessage = []()
	{
		return LOCTEXT("TemplateTextUnbalancedBracingError", "An unbalanced brace was detected. Please ensure that all braces are properly closed.");
	};

	FString TextString = InTokenizedText.ToString();
	for (const TCHAR& Char : TextString)
	{
		if (Char == TEXT('{'))
		{
			++OpenBraceCount;

			// didn't close the last brace
			if (bLastBraceWasOpen)
			{
				OutError = GetUnbalancedBracesMessage();
				break;
			}

			bLastBraceWasOpen = true;
		}
		else if (Char == TEXT('}'))
		{
			++CloseBraceCount;
			bLastBraceWasOpen = false;
		}
	}

	if (OpenBraceCount != CloseBraceCount)
	{
		OutError = GetUnbalancedBracesMessage();
	}

	return OutError.IsEmpty();
}

bool STemplateStringEditableTextBox::ValidateTokenArgs(const FText& InTokenizedText, FText& OutError)
{
	TConstArrayView<FString> ValidArgs = ValidArguments.Get();
	if (ValidArgs.IsEmpty())
	{
		// Always valid if no provided valid args to check against
		return true;
	}

	TArray<FString> FoundArgs;
	if (!ParseArgs(InTokenizedText.ToString(), FoundArgs))
	{
		// No args in tokenized text
		return true;
	}

	// Not particularly performant
	for (const FString& FoundArg : FoundArgs)
	{
		if (!ValidArgs.ContainsByPredicate([FoundArg](const FString& InValidArg)
		{
			return InValidArg.Equals(FoundArg, ESearchCase::IgnoreCase);
		}))
		{
			OutError = LOCTEXT("TemplateTextInvalidArgError", "An argument/token was found that doesn't match any of the provided valid argument names.");
			return false;
		}
	}

	return true;
}

// @see: UE::NamingTokens::Utils::GetTokenKeysFromString
bool STemplateStringEditableTextBox::ParseArgs(const FString& InTokenizedTextString, TArray<FString>& OutArgs)
{
	TSet<FString, FLocKeySetFuncs> Tokens;

	const FString PatternString = TEXT(R"(\{\s*((?:[a-zA-Z0-9_]+\.)*[a-zA-Z0-9_]+)\s*\})");
	const FRegexPattern Pattern(PatternString);
	FRegexMatcher Matcher(Pattern, InTokenizedTextString);

	while (Matcher.FindNext())
	{
		FString Token = Matcher.GetCaptureGroup(1);
		Tokens.Add(Token);
	}

	OutArgs = Tokens.Array();

	return !OutArgs.IsEmpty();
}

void STemplateStringEditableTextBox::OnEditableTextChanged(const FText& InTokenizedText)
{
	ValidateTokenizedText(InTokenizedText);

	OnTokenizedTextChanged.ExecuteIfBound(InTokenizedText);
}

void STemplateStringEditableTextBox::OnEditableTextCommitted(const FText& InTokenizedText, ETextCommit::Type InCommitType)
{
	if (!ValidateTokenizedText(InTokenizedText))
	{
		return;
	}

	if (!TokenizedText.IsBound())
	{
		TokenizedText.Set(InTokenizedText);
	}

	OnTokenizedTextCommitted.ExecuteIfBound(InTokenizedText, InCommitType);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE
