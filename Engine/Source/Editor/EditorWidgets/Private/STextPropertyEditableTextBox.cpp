// Copyright Epic Games, Inc. All Rights Reserved.

#include "STextPropertyEditableTextBox.h"
#include "AssetRegistry/AssetData.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Styling/AppStyle.h"
#include "Misc/PackageName.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Internationalization/StringTable.h"
#include "Internationalization/StringTableCore.h"
#include "Internationalization/StringTableRegistry.h"
#include "Serialization/TextReferenceCollector.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SLinkedBox.h"
#include "SSimpleComboButton.h"

#define LOCTEXT_NAMESPACE "STextPropertyEditableTextBox"

FText STextPropertyEditableTextBox::MultipleValuesText(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"));

#if USE_STABLE_LOCALIZATION_KEYS

void IEditableTextProperty::StaticStableTextId(UObject* InObject, const ETextPropertyEditAction InEditAction, const FString& InTextSource, const FString& InProposedNamespace, const FString& InProposedKey, FString& OutStableNamespace, FString& OutStableKey)
{
	UPackage* Package = InObject ? InObject->GetOutermost() : nullptr;
	StaticStableTextId(Package, InEditAction, InTextSource, InProposedNamespace, InProposedKey, OutStableNamespace, OutStableKey);
}

void IEditableTextProperty::StaticStableTextId(UPackage* InPackage, const ETextPropertyEditAction InEditAction, const FString& InTextSource, const FString& InProposedNamespace, const FString& InProposedKey, FString& OutStableNamespace, FString& OutStableKey)
{
	TextNamespaceUtil::GetTextIdForEdit(InPackage, (TextNamespaceUtil::ETextEditAction)InEditAction, InTextSource, InProposedNamespace, InProposedKey, OutStableNamespace, OutStableKey);
}

#endif // USE_STABLE_LOCALIZATION_KEYS

void STextPropertyEditableStringTableReference::Construct(const FArguments& InArgs, const TSharedRef<IEditableTextProperty>& InEditableTextProperty)
{
	EditableTextProperty = InEditableTextProperty;

	OptionTextFilter = MakeShareable(new FOptionTextFilter(FOptionTextFilter::FItemToStringArray::CreateLambda([](const TSharedPtr<FAvailableStringTable>& InItem, OUT TArray< FString >& StringArray) {
		StringArray.Add(InItem->DisplayName.ToString());
	})));
	KeyTextFilter = MakeShareable(new FKeyTextFilter(FKeyTextFilter::FItemToStringArray::CreateLambda([](const TSharedPtr<FString>& InItem, OUT TArray< FString >& StringArray) {
		StringArray.Add(*InItem);
	})));

	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	HorizontalBox->AddSlot()
		.Padding(0.f)
		[
			SAssignNew(StringTableOptionsCombo, SComboButton)
			.ComboButtonStyle(&InArgs._ComboStyle->ComboButtonStyle)
			.ContentPadding(FMargin(4.0f, 2.0f))
			.OnGetMenuContent(this, &STextPropertyEditableStringTableReference::OnGetStringTableComboOptions)
			.OnComboBoxOpened(this, &STextPropertyEditableStringTableReference::UpdateStringTableComboOptions)
			.CollapseMenuOnParentFocus(true)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &STextPropertyEditableStringTableReference::GetStringTableComboContent)
				.ToolTipText(this, &STextPropertyEditableStringTableReference::GetStringTableComboToolTip)
				.Font(InArgs._Font)
			]
		];

	HorizontalBox->AddSlot()
		.Padding(10.f, 0.f)
		[
			SAssignNew(StringTableKeysCombo, SComboButton)
			.ComboButtonStyle(&InArgs._ComboStyle->ComboButtonStyle)
			.ContentPadding(FMargin(4.0f, 2.0f))
			.IsEnabled(this, &STextPropertyEditableStringTableReference::IsUnlinkEnabled)
			.OnGetMenuContent(this, &STextPropertyEditableStringTableReference::OnGetStringTableKeyOptions)
			.OnComboBoxOpened(this, &STextPropertyEditableStringTableReference::UpdateStringTableKeyOptions)
			.CollapseMenuOnParentFocus(true)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &STextPropertyEditableStringTableReference::GetKeyComboContent)
				.ToolTipText(this, &STextPropertyEditableStringTableReference::GetKeyComboToolTip)
				.Font(InArgs._Font)
			]
		];

	if (InArgs._AllowUnlink)
	{
		HorizontalBox->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.HeightOverride(22.f)
				.WidthOverride(22.f)
				[
					SNew(SButton)
					.ButtonStyle(InArgs._ButtonStyle)
					.ContentPadding(0.f)
					.ToolTipText(LOCTEXT("UnlinkStringTable", "Unlink"))
					.IsEnabled(this, &STextPropertyEditableStringTableReference::IsUnlinkEnabled)
					.OnClicked(this, &STextPropertyEditableStringTableReference::OnUnlinkClicked)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Delete"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			];
	}

	ChildSlot
	[
		HorizontalBox
	];
}

void STextPropertyEditableStringTableReference::OnOptionsFilterTextChanged(const FText& InNewText)
{
	OptionTextFilter->SetRawFilterText(InNewText);
	OptionsSearchBox->SetError(OptionTextFilter->GetFilterErrorText());

	UpdateStringTableComboOptions();
}

void STextPropertyEditableStringTableReference::OnKeysFilterTextChanged(const FText& InNewText)
{
	KeyTextFilter->SetRawFilterText(InNewText);
	KeysSearchBox->SetError(KeyTextFilter->GetFilterErrorText());

	UpdateStringTableKeyOptions();
}

TSharedRef<SWidget> STextPropertyEditableStringTableReference::OnGetStringTableComboOptions()
{
	const FComboButtonStyle& ComboButtonStyle = FCoreStyle::Get().GetWidgetStyle< FComboButtonStyle >("ComboButton");
	return SNew(SBorder)
		.BorderImage(&ComboButtonStyle.MenuBorderBrush)
		.Padding(ComboButtonStyle.MenuBorderPadding)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(OptionsSearchBox, SSearchBox)
				.OnTextChanged(this, &STextPropertyEditableStringTableReference::OnOptionsFilterTextChanged)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
			SNew(SWidgetSwitcher)
			.WidgetIndex_Lambda([this]() { return StringTableComboOptions.IsEmpty() ? 0 : 1; })

				+SWidgetSwitcher::Slot() // Appears when there are no string tables with keys
				.Padding(12.f)
				[
					SNew(STextBlock).Text(LOCTEXT("EmptyStringTableList", "No string tables available"))
				]

				+SWidgetSwitcher::Slot() // Appears when there's a string table with at least a key
				[
					SNew(SBox)
					.Padding(4.f)
					.WidthOverride(280.f)
					.MaxDesiredHeight(600.f)
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.FillHeight(1.f)
						.Padding(0.f, 5.f, 0.f, 0.f)
						[
							SAssignNew(StringTableOptionsList, SListView<TSharedPtr<FAvailableStringTable>>)
							.ListItemsSource(&StringTableComboOptions)
							.SelectionMode(ESelectionMode::Single)
							.OnGenerateRow(this, &STextPropertyEditableStringTableReference::OnGenerateStringTableComboOption)
							.OnSelectionChanged(this, &STextPropertyEditableStringTableReference::OnStringTableComboChanged)
						]
					]
				]
			]
		];
}

TSharedRef<ITableRow> STextPropertyEditableStringTableReference::OnGenerateStringTableComboOption(TSharedPtr<FAvailableStringTable> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		[
			SNew(STextBlock)
			.Text(InItem->DisplayName)
			.ToolTipText(FText::FromName(InItem->TableId))
		];
}

TSharedRef<SWidget> STextPropertyEditableStringTableReference::OnGetStringTableKeyOptions()
{
	const FComboButtonStyle& ComboButtonStyle = FCoreStyle::Get().GetWidgetStyle< FComboButtonStyle >("ComboButton");
	return SNew(SBorder)
		.BorderImage(&ComboButtonStyle.MenuBorderBrush)
		.Padding(ComboButtonStyle.MenuBorderPadding)
		[
			SNew(SBox)
			.Padding(4.f)
			.WidthOverride(280.f)
			.MaxDesiredHeight(600.f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(KeysSearchBox, SSearchBox)
					.OnTextChanged(this, &STextPropertyEditableStringTableReference::OnKeysFilterTextChanged)
				]
				+SVerticalBox::Slot()
				.FillHeight(1.f)
				.Padding(0.f, 5.f, 0.f, 0.f)
				[
					SAssignNew(StringTableKeysList, SListView<TSharedPtr<FString>>)
					.ListItemsSource(&KeyComboOptions)
					.SelectionMode(ESelectionMode::Single)
					.OnGenerateRow(this, &STextPropertyEditableStringTableReference::OnGenerateStringTableKeyOption)
					.OnSelectionChanged(this, &STextPropertyEditableStringTableReference::OnKeyComboChanged)
				]
			]
		];
}

TSharedRef<ITableRow> STextPropertyEditableStringTableReference::OnGenerateStringTableKeyOption(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		[
			SNew(STextBlock)
			.Text(FText::FromString(*InItem))
			.ToolTipText(FText::FromString(*InItem))
		];
}

void STextPropertyEditableStringTableReference::GetTableIdAndKey(FName& OutTableId, FString& OutKey) const
{
	const int32 NumTexts = EditableTextProperty->GetNumTexts();
	if (NumTexts > 0)
	{
		const FText PropertyValue = EditableTextProperty->GetText(0);
		FTextInspector::GetTableIdAndKey(PropertyValue, OutTableId, OutKey);

		// Verify that all texts are using the same string table and key
		for (int32 TextIndex = 1; TextIndex < NumTexts; ++TextIndex)
		{
			FName TmpTableId;
			FString TmpKey;
			if (FTextInspector::GetTableIdAndKey(PropertyValue, TmpTableId, TmpKey) && OutTableId == TmpTableId)
			{
				if (!OutKey.Equals(TmpKey, ESearchCase::CaseSensitive))
				{
					// Not using the same key - clear the key but keep the table and keep enumerating to verify the table on the remaining texts
					OutKey.Reset();
				}
			}
			else
			{
				// Not using a string table, or using a different string table - clear both table ID and key
				OutTableId = FName();
				OutKey.Reset();
				break;
			}
		}
	}
}

void STextPropertyEditableStringTableReference::SetTableIdAndKey(const FName InTableId, const FString& InKey)
{
	const FText TextToSet = FText::FromStringTable(InTableId, InKey);
	if (TextToSet.IsFromStringTable())
	{
		const int32 NumTexts = EditableTextProperty->GetNumTexts();
		for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
		{
			EditableTextProperty->SetText(TextIndex, TextToSet);
		}
	}
}

void STextPropertyEditableStringTableReference::OnStringTableComboChanged(TSharedPtr<FAvailableStringTable> NewSelection, ESelectInfo::Type SelectInfo)
{
	// If it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct && NewSelection.IsValid())
	{
		// Make sure any selected string table asset is loaded
		FName TableId = NewSelection->TableId;
		IStringTableEngineBridge::FullyLoadStringTableAsset(TableId);

		FStringTableConstPtr StringTable = FStringTableRegistry::Get().FindStringTable(TableId);
		if (StringTable.IsValid())
		{
			// Just use the first key when changing the string table
			StringTable->EnumerateSourceStrings([&](const FString& InKey, const FString& InSourceString) -> bool
			{
				SetTableIdAndKey(TableId, InKey);
				return false; // stop enumeration
			});

			StringTableOptionsCombo->SetIsOpen(false);

			OptionsSearchBox->SetText(FText::GetEmpty());
		}
	}
}

void STextPropertyEditableStringTableReference::UpdateStringTableComboOptions()
{
	FName CurrentTableId;
	{
		FString TmpKey;
		GetTableIdAndKey(CurrentTableId, TmpKey);
	}

	TSharedPtr<FAvailableStringTable> SelectedStringTableComboEntry;
	StringTableComboOptions.Reset();

	// Process assets first (as they may currently be unloaded)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName);

		TArray<FAssetData> StringTableAssets;
		AssetRegistryModule.Get().GetAssetsByClass(UStringTable::StaticClass()->GetClassPathName(), StringTableAssets);

		for (const FAssetData& StringTableAsset : StringTableAssets)
		{
			FName StringTableId = *StringTableAsset.GetObjectPathString();
			// Only allow string tables assets that have entries to be visible otherwise unexpected behavior happens for the user
			bool HasEntries = false;
			FStringTableConstPtr StringTable = FStringTableRegistry::Get().FindStringTable(StringTableId);
			if (StringTable.IsValid())
			{
				if (StringTable->IsInternal())
				{
					continue; // can't pick internal string tables
				}

				StringTable->EnumerateSourceStrings([&](const FString& InKey, const FString& InSourceString) -> bool
				{
					HasEntries = true;
					return false; // stop enumeration
				});
			}
			else
			{
				// Asset is currently unloaded, so just assume it has entries
				HasEntries = true;
			}

			if (!HasEntries)
			{
				continue; // continue on to the next string table asset
			}

			TSharedRef<FAvailableStringTable> AvailableStringTableEntry = MakeShared<FAvailableStringTable>();
			AvailableStringTableEntry->TableId = StringTableId;
			AvailableStringTableEntry->DisplayName = FText::FromName(StringTableAsset.AssetName);
			if (StringTableId == CurrentTableId)
			{
				SelectedStringTableComboEntry = AvailableStringTableEntry;
			}
			if (OptionTextFilter->PassesFilter(AvailableStringTableEntry))
			{
				StringTableComboOptions.Add(AvailableStringTableEntry);
			}
		}
	}

	// Process the remaining non-asset string tables now
	FStringTableRegistry::Get().EnumerateStringTables([&](const FName& InTableId, const FStringTableConstRef& InStringTable) -> bool
	{
		if (InStringTable->IsInternal())
		{
			return true; // can't pick internal string tables
		}

		const bool bAlreadyAdded = StringTableComboOptions.ContainsByPredicate([InTableId](const TSharedPtr<FAvailableStringTable>& InAvailableStringTable)
		{
			return InAvailableStringTable->TableId == InTableId;
		});

		bool bHasEntries = false;
		InStringTable->EnumerateSourceStrings([&bHasEntries](const FString& InKey, const FString& InSourceString) -> bool
		{
			bHasEntries = true;
			return false; // stop enumeration
		});

		if (!bAlreadyAdded && bHasEntries)
		{
			TSharedRef<FAvailableStringTable> AvailableStringTableEntry = MakeShared<FAvailableStringTable>();
			AvailableStringTableEntry->TableId = InTableId;
			AvailableStringTableEntry->DisplayName = FText::FromName(InTableId);
			if (InTableId == CurrentTableId)
			{
				SelectedStringTableComboEntry = AvailableStringTableEntry;
			}
			if (OptionTextFilter->PassesFilter(AvailableStringTableEntry))
			{
				StringTableComboOptions.Add(AvailableStringTableEntry);
			}
		}

		return true; // continue enumeration
	});

	StringTableComboOptions.Sort([](const TSharedPtr<FAvailableStringTable>& InOne, const TSharedPtr<FAvailableStringTable>& InTwo)
	{
		return InOne->DisplayName.ToString() < InTwo->DisplayName.ToString();
	});

	StringTableOptionsList->RebuildList();

	if (SelectedStringTableComboEntry.IsValid())
	{
		StringTableOptionsList->SetItemSelection(SelectedStringTableComboEntry, true);
	}
	else
	{
		StringTableOptionsList->ClearSelection();
	}
}

FText STextPropertyEditableStringTableReference::GetStringTableComboContent() const
{
	FName CurrentTableId;
	{
		FString TmpKey;
		GetTableIdAndKey(CurrentTableId, TmpKey);
	}

	return FText::FromString(FPackageName::GetLongPackageAssetName(CurrentTableId.ToString()));
}

FText STextPropertyEditableStringTableReference::GetStringTableComboToolTip() const
{
	FName CurrentTableId;
	{
		FString TmpKey;
		GetTableIdAndKey(CurrentTableId, TmpKey);
	}

	return FText::FromName(CurrentTableId);
}

void STextPropertyEditableStringTableReference::OnKeyComboChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	// If it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct && NewSelection.IsValid())
	{
		FName CurrentTableId;
		{
			FString TmpKey;
			GetTableIdAndKey(CurrentTableId, TmpKey);
		}

		SetTableIdAndKey(CurrentTableId, *NewSelection);

		StringTableKeysCombo->SetIsOpen(false);

		KeysSearchBox->SetText(FText::GetEmpty());
	}
}

void STextPropertyEditableStringTableReference::UpdateStringTableKeyOptions()
{
	FName CurrentTableId;
	FString CurrentKey;
	GetTableIdAndKey(CurrentTableId, CurrentKey);

	TSharedPtr<FString> SelectedKeyComboEntry;
	KeyComboOptions.Reset();

	if (!CurrentTableId.IsNone())
	{
		FStringTableConstPtr StringTable = FStringTableRegistry::Get().FindStringTable(CurrentTableId);
		if (StringTable.IsValid())
		{
			StringTable->EnumerateSourceStrings([&](const FString& InKey, const FString& InSourceString) -> bool
			{
				TSharedRef<FString> KeyComboEntry = MakeShared<FString>(InKey);
				if (InKey.Equals(CurrentKey, ESearchCase::CaseSensitive))
				{
					SelectedKeyComboEntry = KeyComboEntry;
				}
				if (KeyTextFilter->PassesFilter(KeyComboEntry))
				{
					KeyComboOptions.Add(KeyComboEntry);
				}
				return true; // continue enumeration
			});
		}
	}

	KeyComboOptions.Sort([](const TSharedPtr<FString>& InOne, const TSharedPtr<FString>& InTwo)
	{
		return *InOne < *InTwo;
	});

	StringTableKeysList->RebuildList();

	if (SelectedKeyComboEntry.IsValid())
	{
		StringTableKeysList->SetItemSelection(SelectedKeyComboEntry, true);
	}
	else
	{
		StringTableKeysList->ClearSelection();
	}
}

FText STextPropertyEditableStringTableReference::GetKeyComboContent() const
{
	FString CurrentKey;
	{
		FName TmpTableId;
		GetTableIdAndKey(TmpTableId, CurrentKey);
	}

	if (CurrentKey.IsEmpty())
	{
		return LOCTEXT("NoKeyLabel", "No Key");
	}

	return FText::FromString(MoveTemp(CurrentKey));
}

FText STextPropertyEditableStringTableReference::GetKeyComboToolTip() const
{
	return GetKeyComboContent();
}

bool STextPropertyEditableStringTableReference::IsUnlinkEnabled() const
{
	bool bEnabled = false;

	const int32 NumTexts = EditableTextProperty->GetNumTexts();
	for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
	{
		const FText CurrentText = EditableTextProperty->GetText(TextIndex);
		if (CurrentText.IsFromStringTable())
		{
			bEnabled = true;
			break;
		}
	}

	return bEnabled;
}

FReply STextPropertyEditableStringTableReference::OnUnlinkClicked()
{
	const int32 NumTexts = EditableTextProperty->GetNumTexts();
	for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
	{
		const FText CurrentText = EditableTextProperty->GetText(TextIndex);
		if (CurrentText.IsFromStringTable())
		{
			// Make a copy of the FText separate from the string table but generate a new stable namespace and key
			// This prevents problems with properties that disallow empty text (e.g. enum display name)
			FString NewNamespace;
			FString NewKey;
#if USE_STABLE_LOCALIZATION_KEYS
			EditableTextProperty->GetStableTextId(
				TextIndex,
				IEditableTextProperty::ETextPropertyEditAction::EditedKey,
				CurrentText.ToString(),
				FString(),
				FString(),
				NewNamespace,
				NewKey
			);
#else	// USE_STABLE_LOCALIZATION_KEYS
			NewKey = TextNamespaceUtil::GenerateRandomTextKey();
#endif	// USE_STABLE_LOCALIZATION_KEYS
			
			EditableTextProperty->SetText(TextIndex, FText::ChangeKey(NewNamespace, NewKey, CurrentText, CurrentText.GetDevNotes()));
		}
	}

	return FReply::Handled();
}

/** Single row in the advanced text settings/localization menu. Has a similar appearance to a details row in the property editor. */
class STextPropertyEditableOptionRow : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(STextPropertyEditableOptionRow)
		: _IsHeader(false)
		, _ContentHAlign(HAlign_Fill)
		{}
		SLATE_ARGUMENT(bool, IsHeader)
		SLATE_ARGUMENT(EHorizontalAlignment, ContentHAlign)
		SLATE_ATTRIBUTE(FText, Text)
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, TSharedRef<FLinkedBoxManager> InManager)
	{
		InArgs._Content.Widget->SetToolTip(GetToolTip());

		if (InArgs._IsHeader)
		{
			// Header row, text only, fills entire row
			ChildSlot
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
				.Padding(FMargin(0.f, 0.f, 0.f, 1.f))
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
					.BorderBackgroundColor(FSlateColor(FLinearColor::White))
					.Padding(FMargin(12.f, 8.f, 0.f, 8.f))
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
						.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.BoldFont"))
						.Text(InArgs._Text)
						.ToolTip(GetToolTip())
					]
				]
			];
		}
		else
		{
			// Non-header row, has a name column followed by a value widget
			ChildSlot
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SLinkedBox, InManager)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
						.Padding(FMargin(0.f, 0.f, 0.f, 1.f))
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
							.BorderBackgroundColor(this, &STextPropertyEditableOptionRow::GetBackgroundColor)
							.Padding(FMargin(20.f, 3.5f, 0.f, 3.5f))
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
								.Text(InArgs._Text)
								.ToolTip(GetToolTip())
							]
						]
					]
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
					.Padding(FMargin(0.f, 0.f, 0.f, 1.f))
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
						.BorderBackgroundColor(this, &STextPropertyEditableOptionRow::GetBackgroundColor)
						.Padding(FMargin(14.f, 3.5f, 4.f, 3.5f))
						.HAlign(InArgs._ContentHAlign)
						.VAlign(VAlign_Center)
						[
							InArgs._Content.Widget
						]
					]
				]
			];

			// Clear the tooltip from this widget since it's set on the name/value widgets now
			SetToolTip(nullptr);
		}
	}

private:
	FSlateColor GetBackgroundColor() const
	{
		if (IsHovered())
		{
			return FStyleColors::Header;
		}

		return FStyleColors::Panel;
	}
};

void STextPropertyEditableTextBox::Construct(const FArguments& InArgs, const TSharedRef<IEditableTextProperty>& InEditableTextProperty)
{
	EditableTextProperty = InEditableTextProperty;

	TSharedPtr<SHorizontalBox> HorizontalBox;

	const bool bIsPassword = EditableTextProperty->IsPassword();
	bIsMultiLine = EditableTextProperty->IsMultiLineText();
	if (bIsMultiLine)
	{
		ChildSlot
		[
			SAssignNew(HorizontalBox, SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				SNew(SBox)
				.MinDesiredWidth(InArgs._MinDesiredWidth)
				.MaxDesiredHeight(InArgs._MaxDesiredHeight)
				[
					SAssignNew(MultiLineWidget, SMultiLineEditableTextBox)
					.Text(this, &STextPropertyEditableTextBox::GetTextValue)
					.ToolTipText(this, &STextPropertyEditableTextBox::GetToolTipText)
					.Style(InArgs._Style)
					.Font(InArgs._Font)
					.ForegroundColor(InArgs._ForegroundColor)
					.SelectAllTextWhenFocused(false)
					.ClearKeyboardFocusOnCommit(false)
					.OnTextChanged(this, &STextPropertyEditableTextBox::OnTextChanged)
					.OnTextCommitted(this, &STextPropertyEditableTextBox::OnTextCommitted)
					.SelectAllTextOnCommit(false)
					.IsReadOnly(this, &STextPropertyEditableTextBox::IsSourceTextReadOnly)
					.AutoWrapText(InArgs._AutoWrapText)
					.WrapTextAt(InArgs._WrapTextAt)
					.ModiferKeyForNewLine(EModifierKey::Shift)
					//.IsPassword(bIsPassword)
				]
			]
		];

		PrimaryWidget = MultiLineWidget;
	}
	else
	{
		ChildSlot
		[
			SAssignNew(HorizontalBox, SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.MinDesiredWidth(InArgs._MinDesiredWidth)
				[
					SAssignNew(SingleLineWidget, SEditableTextBox)
					.Text(this, &STextPropertyEditableTextBox::GetTextValue)
					.ToolTipText(this, &STextPropertyEditableTextBox::GetToolTipText)
					.Style(InArgs._Style)
					.Font(InArgs._Font)
					.ForegroundColor(InArgs._ForegroundColor)
					.SelectAllTextWhenFocused(true)
					.ClearKeyboardFocusOnCommit(false)
					.OnTextChanged(this, &STextPropertyEditableTextBox::OnTextChanged)
					.OnTextCommitted(this, &STextPropertyEditableTextBox::OnTextCommitted)
					.SelectAllTextOnCommit(true)
					.IsReadOnly(this, &STextPropertyEditableTextBox::IsSourceTextReadOnly)
					.IsPassword(bIsPassword)
				]
			]
		];

		PrimaryWidget = SingleLineWidget;
	}

	const TSharedRef<FLinkedBoxManager> LinkedBoxManager = MakeShared<FLinkedBoxManager>();
	const FSlateFontInfo PropertyNormalFont = FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont");

	HorizontalBox->AddSlot()
		.AutoWidth()
		[
			SNew(SSimpleComboButton)
			.Icon(this, &STextPropertyEditableTextBox::GetAdvancedTextSettingsComboImage)
			.MenuContent()
			[
				SNew(SBox)
				.WidthOverride(340.f)
				.Padding(1)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					[
						SNew(STextPropertyEditableOptionRow, LinkedBoxManager)
						.Text(LOCTEXT("TextLocalizableLabel", "Localize"))
						.ToolTipText(LOCTEXT("TextLocalizableCheckBoxToolTip", "Whether to assign this text a key and allow it to be gathered for localization.\nIf set to false, marks this text as 'culture invariant' to prevent it being gathered for localization."))
						.ContentHAlign(HAlign_Left)
						[
							SNew(SCheckBox)
							.IsEnabled(this, &STextPropertyEditableTextBox::IsCultureInvariantFlagEnabled)
							.IsChecked(this, &STextPropertyEditableTextBox::GetLocalizableCheckState)
							.OnCheckStateChanged(this, &STextPropertyEditableTextBox::HandleLocalizableCheckStateChanged)
						]
					]
					+ SVerticalBox::Slot()
					[
						SNew(STextPropertyEditableOptionRow, LinkedBoxManager)
						.IsHeader(true)
						.Text(LOCTEXT("TextReferencedTextLabel", "Referenced Text"))
					]
					+ SVerticalBox::Slot()
					[
						SNew(STextPropertyEditableOptionRow, LinkedBoxManager)
						.Text(LOCTEXT("TextStringTableLabel", "String Table"))
						.IsEnabled(this, &STextPropertyEditableTextBox::IsTextLocalizable)
						[
							SNew(STextPropertyEditableStringTableReference, InEditableTextProperty)
							.AllowUnlink(true)
							.Font(PropertyNormalFont)
							.IsEnabled(this, &STextPropertyEditableTextBox::CanEdit)
						]
					]
					+ SVerticalBox::Slot()
					[
						SNew(STextPropertyEditableOptionRow, LinkedBoxManager)
						.IsHeader(true)
						.Text(LOCTEXT("TextInlineTextLabel", "Inline Text"))
					]

#if USE_STABLE_LOCALIZATION_KEYS
					+ SVerticalBox::Slot()
					[
						SNew(STextPropertyEditableOptionRow, LinkedBoxManager)
						.Text(LOCTEXT("TextPackageLabel", "Package"))
						.IsEnabled(this, &STextPropertyEditableTextBox::IsTextLocalizable)
						[
							SNew(SEditableTextBox)
							.Text(this, &STextPropertyEditableTextBox::GetPackageValue)
							.Font(PropertyNormalFont)
							.IsReadOnly(true)
						]
					]
#endif // USE_STABLE_LOCALIZATION_KEYS

					+ SVerticalBox::Slot()
					[
						SNew(STextPropertyEditableOptionRow, LinkedBoxManager)
						.Text(LOCTEXT("TextNamespaceLabel", "Namespace"))
						.IsEnabled(this, &STextPropertyEditableTextBox::IsTextLocalizable)
						[
							SAssignNew(NamespaceEditableTextBox, SEditableTextBox)
							.Text(this, &STextPropertyEditableTextBox::GetNamespaceValue)
							.Font(PropertyNormalFont)
							.SelectAllTextWhenFocused(true)
							.ClearKeyboardFocusOnCommit(false)
							.OnTextChanged(this, &STextPropertyEditableTextBox::OnNamespaceChanged)
							.OnTextCommitted(this, &STextPropertyEditableTextBox::OnNamespaceCommitted)
							.SelectAllTextOnCommit(true)
							.IsReadOnly(this, &STextPropertyEditableTextBox::IsIdentityReadOnly)
						]
					]
					+ SVerticalBox::Slot()
					[
						SNew(STextPropertyEditableOptionRow, LinkedBoxManager)
						.Text(LOCTEXT("TextKeyLabel", "Key"))
						.IsEnabled(this, &STextPropertyEditableTextBox::IsTextLocalizable)
						[
							SAssignNew(KeyEditableTextBox, SEditableTextBox)
							.Text(this, &STextPropertyEditableTextBox::GetKeyValue)
							.Font(PropertyNormalFont)
#if USE_STABLE_LOCALIZATION_KEYS
							.SelectAllTextWhenFocused(true)
							.ClearKeyboardFocusOnCommit(false)
							.OnTextChanged(this, &STextPropertyEditableTextBox::OnKeyChanged)
							.OnTextCommitted(this, &STextPropertyEditableTextBox::OnKeyCommitted)
							.SelectAllTextOnCommit(true)
							.IsReadOnly(this, &STextPropertyEditableTextBox::IsIdentityReadOnly)
#else	// USE_STABLE_LOCALIZATION_KEYS
							.IsReadOnly(true)
#endif	// USE_STABLE_LOCALIZATION_KEYS
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextPropertyEditableOptionRow, LinkedBoxManager)
						.Text(LOCTEXT("DevNotesLabel", "Dev Notes"))
						.IsEnabled(this, &STextPropertyEditableTextBox::IsTextLocalizable)
						[
							SNew(SMultiLineEditableTextBox)
							.Text(this, &STextPropertyEditableTextBox::GetDevNotesValue)
							.Font(PropertyNormalFont)
							.ClearKeyboardFocusOnCommit(false)
							.SelectAllTextOnCommit(true)
							.ModiferKeyForNewLine(EModifierKey::Shift)
							.OnTextCommitted(this, &STextPropertyEditableTextBox::OnDevNotesCommitted)
							.IsReadOnly(this, &STextPropertyEditableTextBox::IsIdentityReadOnly)
							.AutoWrapText(true)
						]
					]
				]
			]
		];

	SetEnabled(TAttribute<bool>(this, &STextPropertyEditableTextBox::CanEdit));
}

bool STextPropertyEditableTextBox::IsTextLocalizable() const
{
	// All text need !IsCultureInvariant()
	const int32 NumTexts = EditableTextProperty->GetNumTexts();
	for (int32 Index = 0; Index < NumTexts; ++Index)
	{
		const FText PropertyValue = EditableTextProperty->GetText(Index);
		if (PropertyValue.IsCultureInvariant())
		{
			return false;
		}
	}
	return true;
}

void STextPropertyEditableTextBox::GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth)
{
	if (bIsMultiLine)
	{
		OutMinDesiredWidth = 250.0f;
	}
	else
	{
		OutMinDesiredWidth = 125.0f;
	}

	OutMaxDesiredWidth = 600.0f;
}

bool STextPropertyEditableTextBox::SupportsKeyboardFocus() const
{
	return PrimaryWidget.IsValid() && PrimaryWidget->SupportsKeyboardFocus();
}

FReply STextPropertyEditableTextBox::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	// Forward keyboard focus to our editable text widget
	return FReply::Handled().SetUserFocus(PrimaryWidget.ToSharedRef(), InFocusEvent.GetCause());
}

bool STextPropertyEditableTextBox::CanEdit() const
{
	const bool bIsReadOnly = FTextLocalizationManager::Get().IsLocalizationLocked() || EditableTextProperty->IsReadOnly();
	return !bIsReadOnly;
}

bool STextPropertyEditableTextBox::IsCultureInvariantFlagEnabled() const
{
	return !IsSourceTextReadOnly();
}

bool STextPropertyEditableTextBox::IsSourceTextReadOnly() const
{
	if (!CanEdit())
	{
		return true;
	}

	// We can't edit the source string of external string table references
	const int32 NumTexts = EditableTextProperty->GetNumTexts();
	for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
	{
		const FText TextValue = EditableTextProperty->GetText(TextIndex);
		if (TextValue.IsFromStringTable())
		{
			FName TableId;
			FString Key;
			FTextInspector::GetTableIdAndKey(TextValue, TableId, Key);

			if (FStringTableConstPtr StringTable = FStringTableRegistry::Get().FindStringTable(TableId);
				StringTable && !StringTable->IsInternal())
			{
				return true;
			}
		}
	}

	return false;
}

bool STextPropertyEditableTextBox::IsIdentityReadOnly() const
{
	if (!CanEdit())
	{
		return true;
	}

	// We can't edit the identity of texts that don't gather for localization
	const int32 NumTexts = EditableTextProperty->GetNumTexts();
	for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
	{
		const FText TextValue = EditableTextProperty->GetText(TextIndex);
		if (!TextValue.ShouldGatherForLocalization())
		{
			return true;
		}
	}

	return false;
}

FText STextPropertyEditableTextBox::GetToolTipText() const
{
	FText LocalizedTextToolTip;
	const int32 NumTexts = EditableTextProperty->GetNumTexts();
	if (NumTexts == 1)
	{
		const FText TextValue = EditableTextProperty->GetText(0);

		if (TextValue.IsFromStringTable())
		{
			FName TableId;
			FString Key;
			FTextInspector::GetTableIdAndKey(TextValue, TableId, Key);

			LocalizedTextToolTip = FText::Format(
				LOCTEXT("StringTableTextToolTipFmt", "--- String Table Reference ---\nTable ID: {0}\nKey: {1}"), 
				FText::FromName(TableId), FText::FromString(Key)
				);
		}
		else
		{
			FTextId TextId;
			const FString* SourceString = FTextInspector::GetSourceString(TextValue);

			if (SourceString && TextValue.ShouldGatherForLocalization())
			{
				TextId = FTextInspector::GetTextId(TextValue);
			}

			if (!TextId.IsEmpty())
			{
				check(SourceString);

				const FString Namespace = TextId.GetNamespace().ToString();
				const FString Key = TextId.GetKey().ToString();

				const FString PackageNamespace = TextNamespaceUtil::ExtractPackageNamespace(Namespace);
				const FString TextNamespace = TextNamespaceUtil::StripPackageNamespace(Namespace);

				FFormatNamedArguments LocalizedTextToolTipArgs;
				LocalizedTextToolTipArgs.Add(TEXT("Package"), FText::FromString(PackageNamespace));
				LocalizedTextToolTipArgs.Add(TEXT("Namespace"), FText::FromString(TextNamespace));
				LocalizedTextToolTipArgs.Add(TEXT("Key"), FText::FromString(Key));
				LocalizedTextToolTipArgs.Add(TEXT("Source"), FText::FromString(*SourceString));
				LocalizedTextToolTipArgs.Add(TEXT("Display"), TextValue);

				if (SourceString->Equals(TextValue.ToString(), ESearchCase::CaseSensitive))
				{
					LocalizedTextToolTip = FText::Format(
						LOCTEXT("LocalizedTextNoDisplayToolTipFmt", "--- Localized Text ---\nPackage: {Package}\nNamespace: {Namespace}\nKey: {Key}\nSource: {Source}"),
						LocalizedTextToolTipArgs
						);
				}
				else
				{
					LocalizedTextToolTip = FText::Format(
						LOCTEXT("LocalizedTextWithDisplayToolTipFmt", "--- Localized Text ---\nPackage: {Package}\nNamespace: {Namespace}\nKey: {Key}\nSource: {Source}\nDisplay: {Display}"),
						LocalizedTextToolTipArgs
						);
				}
			}
		}
	}
	
	FText BaseToolTipText = EditableTextProperty->GetToolTipText();
	if (FTextLocalizationManager::Get().IsLocalizationLocked())
	{
		const FText LockdownToolTip = FTextLocalizationManager::Get().IsGameLocalizationPreviewEnabled() 
			? LOCTEXT("LockdownToolTip_Preview", "Localization is locked down due to the active game localization preview")
			: LOCTEXT("LockdownToolTip_Other", "Localization is locked down");
		BaseToolTipText = BaseToolTipText.IsEmptyOrWhitespace() ? LockdownToolTip : FText::Format(LOCTEXT("ToolTipLockdownFmt", "!!! {0} !!!\n\n{1}"), LockdownToolTip, BaseToolTipText);
	}

	if (LocalizedTextToolTip.IsEmptyOrWhitespace())
	{
		return BaseToolTipText;
	}
	if (BaseToolTipText.IsEmptyOrWhitespace())
	{
		return LocalizedTextToolTip;
	}

	return FText::Format(FText::AsCultureInvariant(TEXT("{0}\n\n{1}")), BaseToolTipText, LocalizedTextToolTip);
}

FText STextPropertyEditableTextBox::GetTextValue() const
{
	const int32 NumTexts = EditableTextProperty->GetNumTexts();

	if (NumTexts > 0)
	{
		FString SourceString;
		for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
		{
			const FText TextValue = EditableTextProperty->GetText(TextIndex);
			if (const FString* SourceStringPtr = FTextInspector::GetSourceString(TextValue))
			{
				if (TextIndex == 0)
				{
					SourceString = *SourceStringPtr;
				}
				else if (!SourceString.Equals(*SourceStringPtr, ESearchCase::CaseSensitive))
				{
					return MultipleValuesText;
				}
			}
			else if (TextIndex > 0)
			{
				return MultipleValuesText;
			}
		}
		return FText::AsCultureInvariant(MoveTemp(SourceString));
	}

	return FText::GetEmpty();
}

void STextPropertyEditableTextBox::OnTextChanged(const FText& NewText)
{
	const int32 NumTexts = EditableTextProperty->GetNumTexts();

	FText TextErrorMsg;

	// Don't validate the Multiple Values text if there are multiple properties being set
	if (NumTexts > 0 && (NumTexts == 1 || NewText.ToString().Equals(MultipleValuesText.ToString(), ESearchCase::CaseSensitive)))
	{
		EditableTextProperty->IsValidText(NewText, TextErrorMsg);
	}

	// Update or clear the error message
	SetTextError(TextErrorMsg);
}

void STextPropertyEditableTextBox::OnTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	const int32 NumTexts = EditableTextProperty->GetNumTexts();

	// Allow the first edit to a default value to make invariant text localizable again, 
	// as the value being set will become an override that was likely meant to be localized
	const bool bPreserveCultureInvariance = !EditableTextProperty->IsDefaultValue();

	// Don't commit the Multiple Values text if there are multiple properties being set
	if (NumTexts > 0 && (NumTexts == 1 || !NewText.ToString().Equals(MultipleValuesText.ToString(), ESearchCase::CaseSensitive)))
	{
		FText TextErrorMsg;
		if (EditableTextProperty->IsValidText(NewText, TextErrorMsg))
		{
			// Valid text; clear any error
			SetTextError(FText::GetEmpty());
		}
		else
		{
			// Invalid text; set the error and prevent the new text from being set
			SetTextError(TextErrorMsg);
			return;
		}

		const FString& SourceString = NewText.ToString();
		for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
		{
			const FText PropertyValue = EditableTextProperty->GetText(TextIndex);

			// Only apply the change if the new text is different
			if (PropertyValue.ToString().Equals(NewText.ToString(), ESearchCase::CaseSensitive))
			{
				continue;
			}

			// Is the new text is empty, just use the empty instance
			if (NewText.IsEmpty())
			{
				EditableTextProperty->SetText(TextIndex, FText::GetEmpty());
				continue;
			}

			// Maintain culture invariance when editing the text
			if (bPreserveCultureInvariance && PropertyValue.IsCultureInvariant())
			{
				EditableTextProperty->SetText(TextIndex, FText::AsCultureInvariant(NewText.ToString()));
				continue;
			}

			FString NewNamespace;
			FString NewKey;
#if USE_STABLE_LOCALIZATION_KEYS
			{
				// Get the stable namespace and key that we should use for this property
				const FString* TextSource = FTextInspector::GetSourceString(PropertyValue);
				EditableTextProperty->GetStableTextId(
					TextIndex, 
					IEditableTextProperty::ETextPropertyEditAction::EditedSource, 
					TextSource ? *TextSource : FString(), 
					FTextInspector::GetNamespace(PropertyValue).Get(FString()), 
					FTextInspector::GetKey(PropertyValue).Get(FString()), 
					NewNamespace, 
					NewKey
					);
			}
#else	// USE_STABLE_LOCALIZATION_KEYS
			{
				// We want to preserve the namespace set on this property if it's *not* the default value
				if (!EditableTextProperty->IsDefaultValue())
				{
					// Some properties report that they're not the default, but still haven't been set from a property, so we also check the property key to see if it's a valid GUID before allowing the namespace to persist
					FGuid TmpGuid;
					if (FGuid::Parse(FTextInspector::GetKey(PropertyValue).Get(FString()), TmpGuid))
					{
						NewNamespace = FTextInspector::GetNamespace(PropertyValue).Get(FString());
					}
				}

				NewKey = FGuid::NewGuid().ToString();
			}
#endif	// USE_STABLE_LOCALIZATION_KEYS

			EditableTextProperty->SetText(TextIndex, FText::ChangeKey(NewNamespace, NewKey, NewText, PropertyValue.GetDevNotes()));
		}
	}
}

void STextPropertyEditableTextBox::SetTextError(const FText& InErrorMsg)
{
	if (MultiLineWidget.IsValid())
	{
		MultiLineWidget->SetError(InErrorMsg);
	}

	if (SingleLineWidget.IsValid())
	{
		SingleLineWidget->SetError(InErrorMsg);
	}
}

FText STextPropertyEditableTextBox::GetNamespaceValue() const
{
	const int32 NumTexts = EditableTextProperty->GetNumTexts();

	if (NumTexts > 0)
	{
		FString NamespaceValue;
		for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
		{
			const FText TextValue = EditableTextProperty->GetText(TextIndex);
			if (TOptional<FString> FoundNamespace = FTextInspector::GetNamespace(TextValue))
			{
				FString CleanNamespace = TextNamespaceUtil::StripPackageNamespace(FoundNamespace.GetValue());
				if (TextIndex == 0)
				{
					NamespaceValue = MoveTemp(CleanNamespace);
				}
				else if (!NamespaceValue.Equals(CleanNamespace, ESearchCase::CaseSensitive))
				{
					return MultipleValuesText;
				}
			}
			else if (TextIndex > 0)
			{
				return MultipleValuesText;
			}
		}
		return FText::AsCultureInvariant(MoveTemp(NamespaceValue));
	}

	return FText::GetEmpty();
}

void STextPropertyEditableTextBox::OnNamespaceChanged(const FText& NewText)
{
	FText ErrorMessage;
	const FText ErrorCtx = LOCTEXT("TextNamespaceErrorCtx", "Namespace");
	IsValidIdentity(NewText, &ErrorMessage, &ErrorCtx);

	NamespaceEditableTextBox->SetError(ErrorMessage);
}

void STextPropertyEditableTextBox::OnNamespaceCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (!IsValidIdentity(NewText))
	{
		return;
	}

	const int32 NumTexts = EditableTextProperty->GetNumTexts();

	// Don't commit the Multiple Values text if there are multiple properties being set
	if (NumTexts > 0 && (NumTexts == 1 || NewText.ToString() != MultipleValuesText.ToString()))
	{
		const FString& TextNamespace = NewText.ToString();
		for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
		{
			const FText PropertyValue = EditableTextProperty->GetText(TextIndex);

			// Only apply the change if the new namespace is different - we want to keep the keys stable where possible
			const FString CurrentTextNamespace = TextNamespaceUtil::StripPackageNamespace(FTextInspector::GetNamespace(PropertyValue).Get(FString()));
			if (CurrentTextNamespace.Equals(TextNamespace, ESearchCase::CaseSensitive))
			{
				continue;
			}

			// Get the stable namespace and key that we should use for this property
			FString NewNamespace;
			FString NewKey;
#if USE_STABLE_LOCALIZATION_KEYS
			{
				const FString* TextSource = FTextInspector::GetSourceString(PropertyValue);
				EditableTextProperty->GetStableTextId(
					TextIndex, 
					IEditableTextProperty::ETextPropertyEditAction::EditedNamespace, 
					TextSource ? *TextSource : FString(), 
					TextNamespace, 
					FTextInspector::GetKey(PropertyValue).Get(FString()), 
					NewNamespace, 
					NewKey
					);
			}
#else	// USE_STABLE_LOCALIZATION_KEYS
			{
				NewNamespace = TextNamespace;

				// If the current key is a GUID, then we can preserve that when setting the new namespace
				NewKey = FTextInspector::GetKey(PropertyValue).Get(FString());
				{
					FGuid TmpGuid;
					if (!FGuid::Parse(NewKey, TmpGuid))
					{
						NewKey = FGuid::NewGuid().ToString();
					}
				}
			}
#endif	// USE_STABLE_LOCALIZATION_KEYS

			EditableTextProperty->SetText(TextIndex, FText::ChangeKey(NewNamespace, NewKey, PropertyValue, PropertyValue.GetDevNotes()));
		}
	}
}

FText STextPropertyEditableTextBox::GetKeyValue() const
{
	const int32 NumTexts = EditableTextProperty->GetNumTexts();

	if (NumTexts > 0)
	{
		FString KeyValue;
		for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
		{
			const FText TextValue = EditableTextProperty->GetText(TextIndex);
			if (TOptional<FString> FoundKey = FTextInspector::GetKey(TextValue))
			{
				if (TextIndex == 0)
				{
					KeyValue = FoundKey.GetValue();
				}
				else if (!KeyValue.Equals(FoundKey.GetValue(), ESearchCase::CaseSensitive))
				{
					return MultipleValuesText;
				}
			}
			else if (TextIndex > 0)
			{
				return MultipleValuesText;
			}
		}
		return FText::AsCultureInvariant(MoveTemp(KeyValue));
	}

	return FText::GetEmpty();
}

#if USE_STABLE_LOCALIZATION_KEYS

void STextPropertyEditableTextBox::OnKeyChanged(const FText& NewText)
{
	FText ErrorMessage;
	const FText ErrorCtx = LOCTEXT("TextKeyErrorCtx", "Key");
	const bool bIsValidName = IsValidIdentity(NewText, &ErrorMessage, &ErrorCtx);

	if (NewText.IsEmptyOrWhitespace())
	{
		ErrorMessage = LOCTEXT("TextKeyEmptyErrorMsg", "Key cannot be empty so a new key will be assigned");
	}
	else if (bIsValidName)
	{
		// Valid name, so check it won't cause an identity conflict (only test if we have a single text selected to avoid confusion)
		const int32 NumTexts = EditableTextProperty->GetNumTexts();
		if (NumTexts == 1)
		{
			const FText PropertyValue = EditableTextProperty->GetText(0);

			const FString TextNamespace = FTextInspector::GetNamespace(PropertyValue).Get(FString());
			const FString TextKey = NewText.ToString();

			// Get the stable namespace and key that we should use for this property
			// If it comes back with the same namespace but a different key then it means there was an identity conflict
			FString NewNamespace;
			FString NewKey;
			const FString* TextSource = FTextInspector::GetSourceString(PropertyValue);
			EditableTextProperty->GetStableTextId(
				0,
				IEditableTextProperty::ETextPropertyEditAction::EditedKey,
				TextSource ? *TextSource : FString(),
				TextNamespace,
				TextKey,
				NewNamespace,
				NewKey
				);

			if (TextNamespace.Equals(NewNamespace, ESearchCase::CaseSensitive) && !TextKey.Equals(NewKey, ESearchCase::CaseSensitive))
			{
				ErrorMessage = LOCTEXT("TextKeyConflictErrorMsg", "Identity (namespace & key) is being used by a different text within this package so a new key will be assigned");
			}
		}
	}

	KeyEditableTextBox->SetError(ErrorMessage);
}

void STextPropertyEditableTextBox::OnKeyCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (!IsValidIdentity(NewText))
	{
		return;
	}

	const int32 NumTexts = EditableTextProperty->GetNumTexts();

	// Don't commit the Multiple Values text if there are multiple properties being set
	if (NumTexts > 0 && (NumTexts == 1 || NewText.ToString() != MultipleValuesText.ToString()))
	{
		const FString& TextKey = NewText.ToString();
		for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
		{
			const FText PropertyValue = EditableTextProperty->GetText(TextIndex);

			// Only apply the change if the new key is different - we want to keep the keys stable where possible
			const FString CurrentTextKey = FTextInspector::GetKey(PropertyValue).Get(FString());
			if (CurrentTextKey.Equals(TextKey, ESearchCase::CaseSensitive))
			{
				continue;
			}

			// Get the stable namespace and key that we should use for this property
			FString NewNamespace;
			FString NewKey;
			const FString* TextSource = FTextInspector::GetSourceString(PropertyValue);
			EditableTextProperty->GetStableTextId(
				TextIndex, 
				IEditableTextProperty::ETextPropertyEditAction::EditedKey, 
				TextSource ? *TextSource : FString(), 
				FTextInspector::GetNamespace(PropertyValue).Get(FString()), 
				TextKey, 
				NewNamespace, 
				NewKey
				);

			EditableTextProperty->SetText(TextIndex, FText::ChangeKey(NewNamespace, NewKey, PropertyValue, PropertyValue.GetDevNotes()));
		}
	}
}

FText STextPropertyEditableTextBox::GetPackageValue() const
{
	const int32 NumTexts = EditableTextProperty->GetNumTexts();

	if (NumTexts > 0)
	{
		FString PackageValue;
		for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
		{
			const FText TextValue = EditableTextProperty->GetText(TextIndex);
			if (TOptional<FString> FoundNamespace = FTextInspector::GetNamespace(TextValue))
			{
				FString PackageNamespace = TextNamespaceUtil::ExtractPackageNamespace(FoundNamespace.GetValue());
				if (TextIndex == 0)
				{
					PackageValue = MoveTemp(PackageNamespace);
				}
				else if (!PackageValue.Equals(PackageNamespace, ESearchCase::CaseSensitive))
				{
					return MultipleValuesText;
				}
			}
			else if (TextIndex > 0)
			{
				return MultipleValuesText;
			}
		}
		return FText::AsCultureInvariant(MoveTemp(PackageValue));
	}

	return FText::GetEmpty();
}

#endif // USE_STABLE_LOCALIZATION_KEYS

FText STextPropertyEditableTextBox::GetDevNotesValue() const
{
	FText DevNotes;

	const int32 NumTexts = EditableTextProperty->GetNumTexts();
	if (NumTexts == 1)
	{
		DevNotes = FText::FromString(EditableTextProperty->GetText(0).GetDevNotes());
	}
	else if (NumTexts > 1)
	{
		DevNotes = MultipleValuesText;
	}

	return DevNotes;
}

void STextPropertyEditableTextBox::OnDevNotesCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	const int32 NumTexts = EditableTextProperty->GetNumTexts();

	// Don't commit the Multiple Values text if there are multiple properties being set
	if (NumTexts > 0 && (NumTexts == 1 || NewText.ToString() != MultipleValuesText.ToString()))
	{
		for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
		{
			const FText PropertyValue = EditableTextProperty->GetText(TextIndex);
			const FString Namespace = FTextInspector::GetNamespace(PropertyValue).Get(FString());
			const FString Key = FTextInspector::GetKey(PropertyValue).Get(FString());

			EditableTextProperty->SetText(TextIndex, FText::ChangeKey(Namespace, Key, PropertyValue, NewText.ToString()));
		}
	}
}

ECheckBoxState STextPropertyEditableTextBox::GetLocalizableCheckState() const
{
	TOptional<ECheckBoxState> Result;

	const int32 NumTexts = EditableTextProperty->GetNumTexts();
	for (int32 Index = 0; Index < NumTexts; ++Index)
	{
		const FText PropertyValue = EditableTextProperty->GetText(Index);

		const bool bIsLocalized = !PropertyValue.IsCultureInvariant();
		ECheckBoxState NewState = bIsLocalized ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		if (NewState != Result.Get(NewState))
		{
			return ECheckBoxState::Undetermined;
		}
		Result = NewState;
	}

	return Result.Get(ECheckBoxState::Unchecked);
}

void STextPropertyEditableTextBox::HandleLocalizableCheckStateChanged(ECheckBoxState InCheckboxState)
{
	const int32 NumTexts = EditableTextProperty->GetNumTexts();

	if (InCheckboxState == ECheckBoxState::Checked)
	{
		for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
		{
			const FText PropertyValue = EditableTextProperty->GetText(TextIndex);

			// Assign a key to any currently culture invariant texts
			if (PropertyValue.IsCultureInvariant())
			{
				// Get the stable namespace and key that we should use for this property
				FString NewNamespace;
				FString NewKey;
#if USE_STABLE_LOCALIZATION_KEYS
				EditableTextProperty->GetStableTextId(
					TextIndex,
					IEditableTextProperty::ETextPropertyEditAction::EditedKey,
					PropertyValue.ToString(),
					FString(),
					FString(),
					NewNamespace,
					NewKey
					);
#else // USE_STABLE_LOCALIZATION_KEYS
				NewKey = TextNamespaceUtil::GenerateRandomTextKey();
#endif // USE_STABLE_LOCALIZATION_KEYS

				EditableTextProperty->SetText(TextIndex, FText::AsLocalizable_Advanced(NewNamespace, NewKey, PropertyValue.ToString()));
			}
		}
	}
	else
	{
		for (int32 TextIndex = 0; TextIndex < NumTexts; ++TextIndex)
		{
			const FText PropertyValue = EditableTextProperty->GetText(TextIndex);

			// Clear the identity from any non-culture invariant texts
			if (!PropertyValue.IsCultureInvariant())
			{
				const FString* TextSource = FTextInspector::GetSourceString(PropertyValue);
				EditableTextProperty->SetText(TextIndex, FText::AsCultureInvariant(PropertyValue.ToString()));
			}
		}
	}
}

FText STextPropertyEditableTextBox::GetAdvancedTextSettingsComboToolTip() const
{
	if (IsTextLocalizable())
	{
		return LOCTEXT("AdvancedTextSettingsComboToolTip", "Edit advanced text settings.");
	}
	else
	{
		return LOCTEXT("TextNotLocalizedWarningToolTip", "This text is marked as 'culture invariant' and won't be gathered for localization.\nYou can change this by editing the advanced text settings.");
	}
}

const FSlateBrush* STextPropertyEditableTextBox::GetAdvancedTextSettingsComboImage() const
{
	if (IsTextLocalizable())
	{
		return FAppStyle::Get().GetBrush("LocalizationDashboard.MenuIcon");
	}
	else
	{
		return FCoreStyle::Get().GetBrush("Icons.Warning");
	}
}

bool STextPropertyEditableTextBox::IsValidIdentity(const FText& InIdentity, FText* OutReason, const FText* InErrorCtx) const
{
	const FString InvalidIdentityChars = FString::Printf(TEXT("%s%c%c"), INVALID_NAME_CHARACTERS, TextNamespaceUtil::PackageNamespaceStartMarker, TextNamespaceUtil::PackageNamespaceEndMarker);
	return FName::IsValidXName(InIdentity.ToString(), InvalidIdentityChars, OutReason, InErrorCtx);
}

#undef LOCTEXT_NAMESPACE
