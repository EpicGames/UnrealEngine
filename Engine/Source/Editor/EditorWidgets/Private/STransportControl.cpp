// Copyright Epic Games, Inc. All Rights Reserved.

#include "STransportControl.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Framework/SlateDelegates.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Clipping.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Misc/EnumRange.h"
#include "SlateGlobals.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"

#define LOCTEXT_NAMESPACE "STransportControl"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> STransportControl::MakeTransportControlWidget(ETransportControlWidgetType WidgetType, const FOnMakeTransportWidget& MakeCustomWidgetDelegate)
{
	switch(WidgetType)
	{
	case ETransportControlWidgetType::BackwardEnd:
		return SNew(SButton)
			. ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
			. OnClicked(TransportControlArgs.OnBackwardEnd)
			. Visibility(TransportControlArgs.OnBackwardEnd.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
			. ToolTipText( LOCTEXT("ToFront", "To Front") )
			. ContentPadding(0.0f)
			. IsFocusable(TransportControlArgs.bAreButtonsFocusable)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Image(FAppStyle::Get().GetBrush("Animation.Backward_End"))
			];
	case ETransportControlWidgetType::BackwardStep:
		return SNew(SButton)
			. ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
			. OnClicked(TransportControlArgs.OnBackwardStep)
			. Visibility(TransportControlArgs.OnBackwardStep.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
			. ToolTipText( LOCTEXT("ToPrevious", "To Previous") )
			. ContentPadding(0.0f)
			. IsFocusable(TransportControlArgs.bAreButtonsFocusable)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Image(FAppStyle::Get().GetBrush("Animation.Backward_Step"))
			];
	case ETransportControlWidgetType::BackwardPlay:
		return SAssignNew(BackwardPlayButton, SButton)
			. OnClicked(TransportControlArgs.OnBackwardPlay)
			. Visibility(TransportControlArgs.OnBackwardPlay.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
			. ToolTipText( LOCTEXT("Reverse", "Reverse") )
			. ButtonStyle( FAppStyle::Get(), "Animation.PlayControlsButton" )
			. ContentPadding(0.0f)
			. IsFocusable(TransportControlArgs.bAreButtonsFocusable)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Image( this, &STransportControl::GetBackwardStatusIcon )
			];
	case ETransportControlWidgetType::Record:
		return SAssignNew(RecordButton, SButton)
			. ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
			. OnClicked(TransportControlArgs.OnRecord)
			. Visibility_Lambda([this]() -> EVisibility
			{
				if(!TransportControlArgs.OnRecord.IsBound())
				{
					return EVisibility::Collapsed;
				}
				if(TransportControlArgs.OnGetRecordButtonVisibility.IsBound())
				{
					return TransportControlArgs.OnGetRecordButtonVisibility.Execute();
				}
				return EVisibility::Visible;
			})
			. ToolTipText( this, &STransportControl::GetRecordStatusTooltip )
			. ContentPadding(0.0f)
			. IsFocusable(TransportControlArgs.bAreButtonsFocusable)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Animation.Record"))
				.ColorAndOpacity_Lambda([this]()
				{
					bool bIsRecording = false;
					if (TransportControlArgs.OnGetRecording.IsBound())
					{
						bIsRecording = TransportControlArgs.OnGetRecording.Execute();
					}

					if (bIsRecording)
					{
						return FSlateColor::UseForeground();
					}

					return FSlateColor::UseSubduedForeground();
				})
			];
	case ETransportControlWidgetType::ForwardPlay:
		return SAssignNew(ForwardPlayButton, SButton)
			. OnClicked(TransportControlArgs.OnForwardPlay)
			. Visibility(TransportControlArgs.OnForwardPlay.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
			. ToolTipText( this, &STransportControl::GetForwardStatusTooltip )
			. ButtonStyle( FAppStyle::Get(), "Animation.PlayControlsButton" )
			. ContentPadding(0.0f)
			. IsFocusable(TransportControlArgs.bAreButtonsFocusable)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Image( this, &STransportControl::GetForwardStatusIcon )
			];
	case ETransportControlWidgetType::ForwardStep:
		return SNew(SButton)
			. ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
			. OnClicked(TransportControlArgs.OnForwardStep)
			. Visibility(TransportControlArgs.OnForwardStep.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
			. ToolTipText( LOCTEXT("ToNext", "To Next") )
			. ContentPadding(0.0f)
			. IsFocusable(TransportControlArgs.bAreButtonsFocusable)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Image(FAppStyle::Get().GetBrush("Animation.Forward_Step"))
			];
	case ETransportControlWidgetType::ForwardEnd:
		return SNew(SButton)
			. ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
			. OnClicked(TransportControlArgs.OnForwardEnd)
			. Visibility(TransportControlArgs.OnForwardEnd.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
			. ToolTipText( LOCTEXT("ToEnd", "To End") )
			. ContentPadding(0.0f)
			. IsFocusable(TransportControlArgs.bAreButtonsFocusable)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Image(FAppStyle::Get().GetBrush("Animation.Forward_End"))
			];
	case ETransportControlWidgetType::Loop:
		return SAssignNew(LoopButton, SButton)
			. OnClicked(TransportControlArgs.OnToggleLooping)
			. Visibility(TransportControlArgs.OnGetLooping.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
			. ToolTipText( this, &STransportControl::GetLoopStatusTooltip )
			. ButtonStyle( FAppStyle::Get(), "Animation.PlayControlsButton" )
			. ContentPadding(0.0f)
			. IsFocusable(TransportControlArgs.bAreButtonsFocusable)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Image( this, &STransportControl::GetLoopStatusIcon )
			];
	case ETransportControlWidgetType::Custom:
		if(MakeCustomWidgetDelegate.IsBound())
		{
			return MakeCustomWidgetDelegate.Execute();
		}
		break;
	}

	return nullptr;
}

void STransportControl::Construct( const STransportControl::FArguments& InArgs )
{
	TransportControlArgs = InArgs._TransportArgs;
	
	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::ClipToBounds);

	if(TransportControlArgs.WidgetsToCreate.Num() > 0)
	{
		for(FTransportControlWidget WidgetDesc : TransportControlArgs.WidgetsToCreate)
		{
			TSharedPtr<SWidget> Widget = MakeTransportControlWidget(WidgetDesc.WidgetType, WidgetDesc.MakeCustomWidgetDelegate);
			if(Widget.IsValid())
			{
				HorizontalBox->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					Widget.ToSharedRef()
				];
			}
		}	
	}
	else
	{
		for(ETransportControlWidgetType WidgetType : TEnumRange<ETransportControlWidgetType>())
		{
			TSharedPtr<SWidget> Widget = MakeTransportControlWidget(WidgetType);
			if(Widget.IsValid())
			{
				HorizontalBox->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					Widget.ToSharedRef()
				];
			}
		}
	}

	ChildSlot
	[
		HorizontalBox
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool STransportControl::IsTickable() const
{
	// Only bother if an active timer delegate was provided
	return TransportControlArgs.OnTickPlayback.IsBound() && TransportControlArgs.OnGetPlaybackMode.IsBound();
}

void STransportControl::Tick( float DeltaTime )
{
	const auto PlaybackMode = TransportControlArgs.OnGetPlaybackMode.Execute();
	const bool bIsPlaying = PlaybackMode == EPlaybackMode::PlayingForward || PlaybackMode == EPlaybackMode::PlayingReverse;

	if ( bIsPlaying && !ActiveTimerHandle.IsValid() )
	{
		ActiveTimerHandle = RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &STransportControl::TickPlayback ) );
	}
	else if ( !bIsPlaying && ActiveTimerHandle.IsValid() )
	{
		UnRegisterActiveTimer( ActiveTimerHandle.Pin().ToSharedRef() );
	}
}

const FSlateBrush* STransportControl::GetForwardStatusIcon() const
{
	EPlaybackMode::Type PlaybackMode = EPlaybackMode::Stopped;
	if (TransportControlArgs.OnGetPlaybackMode.IsBound())
	{
		PlaybackMode = TransportControlArgs.OnGetPlaybackMode.Execute();
	}

	bool bIsRecording = false;
	if (TransportControlArgs.OnGetRecording.IsBound())
	{
		bIsRecording = TransportControlArgs.OnGetRecording.Execute();
	}

	if ( PlaybackMode == EPlaybackMode::PlayingForward)
	{
		return FAppStyle::Get().GetBrush("Animation.Pause");
	}

	return FAppStyle::Get().GetBrush("Animation.Forward");
}

FText STransportControl::GetForwardStatusTooltip() const
{
	if (TransportControlArgs.OnGetPlaybackMode.IsBound() &&
		TransportControlArgs.OnGetPlaybackMode.Execute() == EPlaybackMode::PlayingForward)
	{
		return LOCTEXT("Pause", "Pause");
	}

	return LOCTEXT("Play", "Play");
}

const FSlateBrush* STransportControl::GetRecordStatusIcon() const
{
	bool bIsRecording = false;
	if (TransportControlArgs.OnGetRecording.IsBound())
	{
		bIsRecording = TransportControlArgs.OnGetRecording.Execute();
	}

	if (bIsRecording)
	{
		return FAppStyle::Get().GetBrush("Animation.Recording");
	}

	return FAppStyle::Get().GetBrush("Animation.Record");
}

FText STransportControl::GetRecordStatusTooltip() const
{
	if (TransportControlArgs.OnGetRecording.IsBound() &&
		TransportControlArgs.OnGetRecording.Execute())
	{
		return LOCTEXT("StopRecording", "Stop Recording");
	}

	return LOCTEXT("Record", "Record");
}

const FSlateBrush* STransportControl::GetBackwardStatusIcon() const
{
	if (TransportControlArgs.OnGetPlaybackMode.IsBound() &&
		TransportControlArgs.OnGetPlaybackMode.Execute() == EPlaybackMode::PlayingReverse)
	{
		return FAppStyle::Get().GetBrush("Animation.Pause");
	}

	return FAppStyle::Get().GetBrush("Animation.Backward");
}

const FSlateBrush* STransportControl::GetLoopStatusIcon() const
{
	if (TransportControlArgs.OnGetLooping.IsBound() &&
		TransportControlArgs.OnGetLooping.Execute())
	{
		return FAppStyle::Get().GetBrush("Animation.Loop.Enabled");
	}

	return FAppStyle::Get().GetBrush("Animation.Loop.Disabled");
}

FText STransportControl::GetLoopStatusTooltip() const
{
	if (TransportControlArgs.OnGetLooping.IsBound() &&
		TransportControlArgs.OnGetLooping.Execute())
	{
		return LOCTEXT("Looping", "Looping");
	}

	return LOCTEXT("NoLooping", "No Looping");
}

EActiveTimerReturnType STransportControl::TickPlayback( double InCurrentTime, float InDeltaTime )
{
	TransportControlArgs.OnTickPlayback.Execute( InCurrentTime, InDeltaTime );
	return EActiveTimerReturnType::Continue;
}

#undef LOCTEXT_NAMESPACE
