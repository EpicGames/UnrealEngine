// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITransportControl.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "TickableEditorObject.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

class FActiveTimerHandle;
class SButton;
struct FSlateBrush;

class STransportControl : public ITransportControl, public FTickableEditorObject
{
public:
	SLATE_BEGIN_ARGS(STransportControl)
		: _TransportArgs() {}

		SLATE_ARGUMENT(FTransportControlArgs, TransportArgs)
	SLATE_END_ARGS()
		
	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs );

	virtual ~STransportControl() = default;

	using SWidget::Tick;

	// Begin FTickableObjectBase implementation
	virtual bool IsTickable() const override;
	virtual void Tick( float DeltaTime ) override;
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT( STransportControl, STATGROUP_Tickables ); }
	// End FTickableObjectBase
private:
	const FSlateBrush* GetForwardStatusIcon() const;
	FText GetForwardStatusTooltip() const;
	const FSlateBrush* GetRecordStatusIcon() const;
	FText GetRecordStatusTooltip() const;
	const FSlateBrush* GetBackwardStatusIcon() const;
	const FSlateBrush* GetLoopStatusIcon() const;
	FText GetLoopStatusTooltip() const;

	/** Executes the OnTickPlayback delegate */
	EActiveTimerReturnType TickPlayback( double InCurrentTime, float InDeltaTime );

	FReply OnToggleLooping();

	/** Make default transport control widgets */
	TSharedPtr<SWidget> MakeTransportControlWidget(ETransportControlWidgetType WidgetType, const FOnMakeTransportWidget& MakeCustomWidgetDelegate = FOnMakeTransportWidget());

private:
	/** The handle to the active timer */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	/** Whether the active timer is currently registered */
	bool bIsActiveTimerRegistered;

	FTransportControlArgs TransportControlArgs;

	TSharedPtr<SButton> ForwardPlayButton;
	TSharedPtr<SButton> BackwardPlayButton;
	TSharedPtr<SButton> RecordButton;
	TSharedPtr<SButton> LoopButton;
};
