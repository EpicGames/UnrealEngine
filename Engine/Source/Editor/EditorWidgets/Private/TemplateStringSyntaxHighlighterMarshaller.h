// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EditorWidgetsStyle.h"
#include "Framework/Text/SyntaxHighlighterTextLayoutMarshaller.h"
#include "Framework/Text/SyntaxTokenizer.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"

class FTextLayout;

/**
 * Get/set the raw text to/from a text layout, and also inject syntax highlighting for our rich-text markup
 */
class FTemplateStringSyntaxHighlighterMarshaller : public FSyntaxHighlighterTextLayoutMarshaller
{
public:
	struct FSyntaxTextStyle
	{
		FSyntaxTextStyle()
			: NormalTextStyle(FEditorWidgetsStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.Template.Normal"))
			, ArgumentTextStyle(FEditorWidgetsStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.Template.Argument"))
		{
		}

		FSyntaxTextStyle(const FTextBlockStyle& InNormalTextStyle, const FTextBlockStyle& InArgumentTextStyle)
			: NormalTextStyle(InNormalTextStyle)
			, ArgumentTextStyle(InArgumentTextStyle)
		{
		}

		FTextBlockStyle NormalTextStyle;
		FTextBlockStyle ArgumentTextStyle;
	};

	static TSharedRef<FTemplateStringSyntaxHighlighterMarshaller> Create(const FSyntaxTextStyle& InSyntaxTextStyle);

	virtual ~FTemplateStringSyntaxHighlighterMarshaller() override = default;

protected:
	// Allows MakeShared with private constructor
	friend class SharedPointerInternals::TIntrusiveReferenceController<FTemplateStringSyntaxHighlighterMarshaller, ESPMode::ThreadSafe>;
	
	virtual void ParseTokens(const FString& SourceString, FTextLayout& TargetTextLayout, TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines) override;

	FTemplateStringSyntaxHighlighterMarshaller(TSharedPtr<ISyntaxTokenizer> InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle);

	/** Styles used to display the text */
	FSyntaxTextStyle SyntaxTextStyle;
};
