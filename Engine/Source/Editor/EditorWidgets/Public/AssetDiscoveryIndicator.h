// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace EAssetDiscoveryIndicatorScaleMode
{
	enum Type
	{
		Scale_None,
		Scale_Vertical,
		Scale_Horizontal,
		Scale_Both
	};
};
