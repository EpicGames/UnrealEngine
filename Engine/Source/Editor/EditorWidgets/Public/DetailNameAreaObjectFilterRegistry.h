// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API EDITORWIDGETS_API

class UClass;

namespace UE::EditorWidgets
{

class IDetailNameAreaObjectFilter
{
public:
	virtual ~IDetailNameAreaObjectFilter() = default;
	virtual FName GetFilterName() const = 0;
	virtual void FilterSelectedObjectsForNameArea(TArray<TWeakObjectPtr<UObject>>& InOutObjects) = 0;
};

class FDetailNameAreaObjectFilterRegistry
{
public:
	
	UE_API void RegisterDetailNameAreaObjectFilter(TUniquePtr<IDetailNameAreaObjectFilter>&& NewNameAreaFilter);
	UE_API void UnregisterDetailNameAreaObjectFilter(const FName& FilterName);

	UE_API void ApplyAllFiltersToSelectedObjects(TArray<TWeakObjectPtr<UObject>>& InOutObjects) const;

private:

	TMap<FName, TUniquePtr<IDetailNameAreaObjectFilter>> ObjectFilters;
};

} // end namespace UE::EditorWidgets

#undef UE_API
