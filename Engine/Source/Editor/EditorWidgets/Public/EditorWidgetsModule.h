// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDiscoveryIndicator.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

class FName;
class ITransportControl;
class SWidget;
class UObject;
struct FTransportControlArgs;

namespace UE::EditorWidgets
{
class FObjectNameEditSinkRegistry;
class FDetailNameAreaObjectFilterRegistry;
}

/** Interface for the widget that wraps an editable text box for viewing the names of objects or editing the labels of actors */
class IObjectNameEditableTextBox : public SCompoundWidget
{
};

/**
 * Editor Widgets module
 */
class FEditorWidgetsModule : public IModuleInterface
{
public:

	/**
	 * Called right after the plugin DLL has been loaded and the plugin object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the plugin is unloaded, right before the plugin object is destroyed.
	 */
	virtual void ShutdownModule();

	/**
	 * Creates a new text box for viewing the names of objects or editing the labels of actors
	 *
 	 * @param Objects		The list of objects that the editable text box shows/edits the names
	 * @return The new editable text box
	 */
	virtual TSharedRef<IObjectNameEditableTextBox> CreateObjectNameEditableTextBox(const TArray<TWeakObjectPtr<UObject>>& Objects);

	/**
	 * Creates a widget that visualizes the asset discovery progress and collapses away when complete
	 *
	 * @param ScaleMode		The direction the widget will scale when animating in and out
 	 * @param Padding		The padding to apply to the widget internally
	 * @return The indicator widget
	 */
	virtual TSharedRef<SWidget> CreateAssetDiscoveryIndicator(EAssetDiscoveryIndicatorScaleMode::Type ScaleMode = EAssetDiscoveryIndicatorScaleMode::Scale_None, FMargin Padding = FMargin(0), bool bFadeIn = true);
	
	/**
	 * Creates a widget that allows play/pause, and general time controls
	 *
	 * @param Args The arguments used to generate the widget
	 * @return The transport control widget
	 */
	virtual TSharedRef<ITransportControl> CreateTransportControl(const FTransportControlArgs& Args);

	/**
	 * Gets the registry of name edit 'sinks' used for getting and editing object display names
	 * 
	 * @return The registry
	 */
	virtual TSharedRef<UE::EditorWidgets::FObjectNameEditSinkRegistry> GetObjectNameEditSinkRegistry() const;

	virtual TSharedRef<UE::EditorWidgets::FDetailNameAreaObjectFilterRegistry> GetDetailNameAreaObjectFilterRegistry() const;

	/** Editor Widgets app identifier string */
	static const FName EditorWidgetsAppIdentifier;

private:
	/** Registry for name interfaces */
	TSharedPtr<UE::EditorWidgets::FObjectNameEditSinkRegistry> ObjectNameEditSinkRegistry;

	TSharedPtr<UE::EditorWidgets::FDetailNameAreaObjectFilterRegistry> DetailNameAreaObjectFilterRegistry;
};
