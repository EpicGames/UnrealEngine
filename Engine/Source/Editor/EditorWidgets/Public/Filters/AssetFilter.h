// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Filters/FilterBase.h"

#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "UObject/NameTypes.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/IAssetRegistry.h"

/* A filter used by SFilterBar that wraps all the back-end filters into one FFilterBase for convenience */
template<typename FilterType>
class FAssetFilter : public FFilterBase<FilterType>
{
public:

	/* A delegate that the user can use to specify how to compare their Filter Item to a list of Asset Types represented as FNames */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FCompareItemWithClassNames, FilterType /* InItem */, const TSet<FTopLevelAssetPath>& /* InClassNames */);

	/* A delegate that the user can use to specify how to convert their Filter Item into an FAssetData
	 * @return: true if the Filter Item was successfully converted, false if not (the current item will not be tested for asset filters if so)
	 */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FConvertItemToAssetData, FilterType /* InItem */, FAssetData& /* OutAssetData */);

	
	FAssetFilter() :
		FFilterBase<FilterType>(nullptr)
	{
		
	}

	void SetBackendFilter(const FARFilter& InBackendFilter)
	{
		// Clear the current compiled filter
		CompiledBackendFilter.Clear();

		// Compile the filter and store it
		IAssetRegistry::Get()->CompileFilter(InBackendFilter, CompiledBackendFilter);
	}
	
	void SetComparisonFunction(FCompareItemWithClassNames InOnCompareItem)
	{
		OnCompareItem = InOnCompareItem;
	}

	void SetConversionFunction(FConvertItemToAssetData InOnConvertItem)
	{
		OnConvertItem = InOnConvertItem;
	}

	/** Returns the system name for this filter */
	virtual FString GetName() const override
	{
		return FString("FAssetFilter");
	}

	/** Returns the human readable name for this filter */
	virtual FText GetDisplayName() const override
	{
		return FText::GetEmpty(); // This filter should never be displayed in the Filter Bar
	}


	/** Returns the tooltip for this filter, shown in the filters menu */
	virtual FText GetToolTipText() const override
	{
		return FText::GetEmpty(); // This filter should never be displayed in the Filter Bar
	}


	/** Returns the color this filter button will be when displayed as a button */
	virtual FLinearColor GetColor() const override
	{
		return FLinearColor();
	}

	/** Returns the name of the icon to use in menu entries */
	virtual FName GetIconName() const override
	{
		return NAME_None;
	}

	/** Returns true if the filter should be in the list when disabled and not in the list when enabled */
	virtual bool IsInverseFilter() const override
	{
		return false;
	}

	virtual bool PassesFilter( FilterType InItem ) const override
	{
		 // The Conversion Function is preferred if it is specified
		if(OnConvertItem.IsBound())
		{
			// Convert the item to an FAssetData
			FAssetData AssetData;
			bool bWasConverted = OnConvertItem.Execute(InItem, AssetData);

			// Skip the current item if it wasn't converted
			if(!bWasConverted)
			{
				/* If the current item does not want to be compared with Asset Type Filters, return true if there are no
				 * asset type filters active, since we want to show it then. Return false if there are any filters active
				 * since we only want to show items that pass the filters if so */
				return CompiledBackendFilter.IsEmpty();
			}
			
			// Use the AssetRegistry to check if the Item passes the compiled filter
			return IAssetRegistry::Get()->IsAssetIncludedByFilter(AssetData, CompiledBackendFilter);
		}
		// If not, try to use the comparison function
		else if(OnCompareItem.IsBound())
		{
			// If there are no filters active just return true
			if(CompiledBackendFilter.IsEmpty())
			{
				return true;
			}
			
			return OnCompareItem.Execute(InItem, CompiledBackendFilter.ClassPaths);
		}

		return false;
	}

	// Functionality that is not needed by this filter
	
	/** Notification that the filter became active or inactive */
	virtual void ActiveStateChanged(bool bActive) override
	{
		
	}

	/** Called when the right-click context menu is being built for this filter */
	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override
	{
		
	}
	
	/** Can be overriden for custom FilterBar subclasses to save settings, currently not implemented in any gneeric Filter Bar */
	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const override
	{
		
	}

	/** Can be overriden for custom FilterBar subclasses to load settings, currently not implemented in any gneeric Filter Bar */
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override
	{
		
	}
	
protected:

	/* The compiled BackendFilter that contains all the Asset Filters attached to this Filter */
	FARCompiledFilter CompiledBackendFilter;

	/* Delegate used to compare an item to a UClass represented by an FName */
	FCompareItemWithClassNames OnCompareItem;

	/* Delegate used to convert an item to an FAssetData */
	FConvertItemToAssetData OnConvertItem;
	
};
