// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetRegistry/ARFilter.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "UObject/TopLevelAssetPath.h"

#define UE_API EDITORWIDGETS_API

class FFilterCategory;
class IAssetTypeActions;
class UAssetDefinition;
struct FAssetFilterData;
class UClass;
struct FARFilter;

/** Helper Class that can be used to provide custom class filters to the Filter Widget
 *  Using this for Type Filters ensures that the type filters get OR'd together
 *  For example: filtering for StaticMeshActor and DirectionalLight will show assets that belong
 *  to either of those types.
 */
class FCustomClassFilterData
{
public:

	/* You can provide an IAssetTypeActions for your type to get all the information from that */
	UE_API FCustomClassFilterData(UAssetDefinition* InAssetDefinition, const FAssetFilterData& InAssetTypeAction);

	/* Or you can provide a UClass, a Category and a Color to identify the filter */
	UE_API FCustomClassFilterData(UClass* InClass, TSharedPtr<FFilterCategory> InCategory, FLinearColor InColor);

	/* Unlike normal filters, Type Filters are allowed to belong to multiple categories */
	UE_API void AddCategory(TSharedPtr<FFilterCategory> InCategory);
	
	/* Get the UClass associated with this filter */
	UE_API UClass* GetClass() const;

	/* Get the Catgories this filter belongs to */
	UE_API TArray<TSharedPtr<FFilterCategory>> GetCategories() const;

	/* Get the color of this filter */
	UE_API FLinearColor GetColor() const;

	/** Add this filter to the input BackendFilter */
	UE_API void BuildBackendFilter(FARFilter& OutFilter);

	/** Get the logical name of this filter class */
	UE_API FText GetName() const;

	/** Get the display name of this filter class */
	UE_API FString GetFilterName() const;

	/**
	 * Returns class path name as a package + class FName pair
	 */
	UE_API FTopLevelAssetPath GetClassPathName() const;

private:

	/** The Class the filter is associated with (if it does not have an AssetTypeAction) */
	TWeakObjectPtr<UClass> Class;

	/** The Filter name */
	FString FilterName;
	
	/** The Filter display name */
	FText FilterDisplayName;

	/** The class path of the class associated with the filter */
	FTopLevelAssetPath ClassPathName;

	/** The actual asset registry filter. */
	FARFilter Filter;

	/** The Categories the filter will show up under */
	TArray<TSharedPtr<FFilterCategory>> Categories;

	/** The Color of the filter (if it does not have an AssetTypeAction to get the color from) */
	FLinearColor Color;

	/** An optional AssetDefinition to get information about the filter class */
	TWeakObjectPtr<UAssetDefinition> AssetDefinitionPtr;
};

#undef UE_API
