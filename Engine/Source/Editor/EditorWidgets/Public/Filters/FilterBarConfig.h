// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "EditorConfigBase.h"
#include "Filters/CustomTextFilters.h"
#include "Filters/SBasicFilterBar.h"
#include "Filters/SCustomTextFilterDialog.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "FilterBarConfig.generated.h"

#define UE_API EDITORWIDGETS_API

class FName;
class FString;
class UObject;

USTRUCT()
struct FCustomTextFilterState
{
	GENERATED_BODY()
	
public:

	/* Whether the custom filter is checked, i.e visible in the filter bar */
	UPROPERTY()
	bool bIsChecked = false;

	/* Whether the custom filter is active, i.e visible and enabled in the filter bar */
	UPROPERTY()
	bool bIsActive = false;

	/* The data inside the custom text filter */
	UPROPERTY()
	FCustomTextFilterData FilterData;
};

USTRUCT()
struct FFilterBarSettings
{
	GENERATED_BODY()

public:

	/** Map of currently visible custom filters, along with their enabled state */
	UPROPERTY()
	TMap<FString, bool> CustomFilters;

	/** Map of currently visible asset type filters, along with their enabled state */
	UPROPERTY()
	TMap<FString, bool> TypeFilters;

	/** Array of custom text filters the user has created */
	UPROPERTY()
	TArray<FCustomTextFilterState> CustomTextFilters;

	UPROPERTY()
	bool bIsLayoutSaved = false;

	UPROPERTY()
	EFilterBarLayout FilterBarLayout = EFilterBarLayout::Horizontal;

	/** Whether to filter paths recursively. Also @see UContentBrowserSettings::FilterRecursively */
	UPROPERTY()
	bool bFilterRecursively = true;

	/** Whether filters are sorted and displayed alphabetically or in the order they were added */
	UPROPERTY()
	bool bSortFiltersAlphabetically = true;

	void Empty()
	{
		CustomFilters.Empty();
		TypeFilters.Empty();
		CustomTextFilters.Empty();
	}
};

UCLASS(MinimalAPI, EditorConfig="FilterBar")
class UFilterBarConfig : public UEditorConfigBase
{
	GENERATED_BODY()
	
public:

	static UE_API void Initialize();
	static UFilterBarConfig* Get() { return Instance; }
	
	UPROPERTY(meta=(EditorConfig))
	TMap<FName, FFilterBarSettings> FilterBars;
	
private:

	static UE_API TObjectPtr<UFilterBarConfig> Instance;
};

#undef UE_API
