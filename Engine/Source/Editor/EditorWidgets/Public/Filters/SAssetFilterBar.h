// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionRegistry.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetToolsModule.h"
#include "AssetTypeCategories.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Factories.h"
#include "Filters/CustomClassFilterData.h"
#include "Filters/CustomTextFilters.h"
#include "Filters/FilterBarConfig.h"
#include "Filters/FilterBase.h"
#include "Filters/SBasicFilterBar.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/NamePermissionList.h"
#include "Modules/ModuleManager.h"
#include "SlateGlobals.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Misc/AssetFilterData.h"

#include "SAssetFilterBar.generated.h"

#define UE_API EDITORWIDGETS_API

class SFilterSearchBox;
class SWidget;

#define LOCTEXT_NAMESPACE "FilterBar"

/** Delegate that subclasses can use to specify classes to not include in this filter
 * Returning false for a class will prevent it from showing up in the add filter dropdown
 */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnFilterAssetType, UClass* /* Test Class */);

/* Delegate used by SAssetFilterBar to populate the add filter menu */
DECLARE_DELEGATE_ThreeParams(FOnPopulateAddAssetFilterMenu, UToolMenu*, TSharedPtr<FFilterCategory>, FOnFilterAssetType)

namespace UE::Editor::Widgets
{
	/** A local struct to describe a category in the filter menu */
	struct FFilterCategoryMenu
	{
		/** The Classes that belong to this category */
		TArray<TSharedPtr<FCustomClassFilterData>> Classes;

		/** Menu section */
		FName SectionExtensionHook;
		FText SectionHeading;

		UE_API FFilterCategoryMenu(const FName& InSectionExtensionHook, const FText& InSectionHeading);
	};

	EDITORWIDGETS_API TMap<TSharedPtr<FFilterCategory>, FFilterCategoryMenu> BuildCategoryToMenuMap(
		TMap<FName, TSharedPtr<FFilterCategory>>& InAssetFilterCategories,
		TConstArrayView<TSharedRef<FCustomClassFilterData>> InCustomClassFilters,
		const FOnFilterAssetType& InOnFilterAssetType);
}

/** ToolMenuContext that is used to create the Add Filter Menu */
UCLASS(MinimalAPI)
class UAssetFilterBarContext : public UObject
{
	GENERATED_BODY()

public:
	TSharedPtr<FFilterCategory> MenuExpansion;
	FOnPopulateAddAssetFilterMenu PopulateFilterMenu;
	FOnFilterAssetType OnFilterAssetType;
	FOnExtendAddFilterMenu OnExtendAddFilterMenu;
};

/** A non-templated base class for SAssetFilterBar, where functionality that does not depend on the type of item
 *  being filtered lives
 */
class FFilterBarBase
{
protected:

	/** Get a mutable version of this filter bar's config */
	UE_API FFilterBarSettings* GetMutableConfig();

	/** Get a const version of this filter bar's config */
	UE_API const FFilterBarSettings* GetConstConfig() const;

	/** Save this filte bar's data to the config file */
	UE_API void SaveConfig();

	/** Initialize and load this filter bar's config */
	UE_API void InitializeConfig();

protected:

	/* Unique name for this filter bar */
	FName FilterBarIdentifier;
};


/**
* An Asset Filter Bar widget, which can be used to filter items of type [FilterType] given a list of custom filters along with built in support for AssetType filters
* @see SFilterBar in EditorWidgets, which is a simplified version of this widget for most use cases which is probably what you want to use
* @see SBasicFilterBar in ToolWidgets if you want a simple FilterBar without AssetType Filters (usable in non editor builds)
* @see SFilterList in ContentBrowser for an extremely complex example of how to use this widget, though for most cases you probably want to use SFilterBar
* NOTE: The filter functions create copies, so you want to use SAssetFilterBar<TSharedPtr<ItemType>> etc instead of SAssetFilterBar<ItemType> when possible
* NOTE: You will need to also add "ToolWidgets" as a dependency to your module to use this widget
* Sample Usage:
*		SAssignNew(MyFilterBar, SAssetFilterBar<FText>)
*		.OnFilterChanged() // A delegate for when the list of filters changes
*		.CustomFilters() // An array of filters available to this FilterBar (@see FGenericFilter to create simple delegate based filters)
*
* Use the GetAllActiveFilters() and GetCombinedBackendFilter() functions to get all the custom and asset filters respectively
* NOTE: GetCombinedBackendFilter returns an FARFilter, and it is on the user of this widget to compile it/use it to filter their items.
*		If you want more straightforward filtering, look at SFilterBar which streamlines this
* Use MakeAddFilterButton() to make the button that summons the dropdown showing all the filters
* Sample Usage:
*  void OnFilterChangedDelegate()
*  {
*		TSharedPtr< TFilterCollection<FilterType> > ActiveFilters = MyFilterBar->GetAllActiveFilters();
*		FARFilter CombinedBackEndFilter = MyFilterBar->GetCombinedBackendFilter()
*		TArray<FilterType> MyUnfilteredItems;
*		TArray<FilterType> FilteredItems;
*		
*		for(FilterType& MyItem : MyUnfilteredItems)
*		{
*			if(CompileAndRunFARFilter(CombinedBackEndFilter, MyItem) && ActiveFilters.PassesAllFilters(MyItem))
*			{
*				FilteredItems.Add(MyItem);
*			}
*		}
*  }
*/
template<typename FilterType>
class SAssetFilterBar : public SBasicFilterBar<FilterType>, public FFilterBarBase
{
public:
	
	using FOnFilterChanged = typename SBasicFilterBar<FilterType>::FOnFilterChanged;
	using FCreateTextFilter = typename SBasicFilterBar<FilterType>::FCreateTextFilter;
	
	SLATE_BEGIN_ARGS( SAssetFilterBar<FilterType> )
		: _FilterMenuName(FName("FilterBar.FilterMenu"))
		, _UseDefaultAssetFilters(true)
		, _FilterBarLayout(EFilterBarLayout::Horizontal)
		, _CanChangeOrientation(false)
		, _FilterPillStyle(EFilterPillStyle::Default)
		, _DefaultMenuExpansionCategory(EAssetCategoryPaths::Basic)
		, _bUseSectionsForCustomCategories(false)
		{
		
		}

		/** Delegate for when filters have changed */
		SLATE_EVENT( FOnFilterChanged, OnFilterChanged )
	
		/** Delegate to extend the Add Filter dropdown */
		SLATE_EVENT( FOnExtendAddFilterMenu, OnExtendAddFilterMenu )
	
		/** Initial List of Custom Filters that will be added to the AddFilter Menu */
		SLATE_ARGUMENT( TArray<TSharedRef<FFilterBase<FilterType>>>, CustomFilters)

		/** Initial List of Custom Class filters that will be added to the AddFilter Menu */
		SLATE_ARGUMENT( TArray<TSharedRef<FCustomClassFilterData>>, CustomClassFilters)

		/** A delegate to create a TTextFilter for FilterType items. If provided, will allow creation of custom text filters
		 *  from the filter dropdown menu.
		 */
		SLATE_ARGUMENT(FCreateTextFilter, CreateTextFilter)
			
		/** An SFilterSearchBox that can be attached to this filter bar. When provided along with a CreateTextFilter
		 *  delegate, allows the user to save searches from the Search Box as text filters for the filter bar.
		 *	NOTE: Will bind a delegate to SFilterSearchBox::OnClickedAddSearchHistoryButton
		 */
		SLATE_ARGUMENT(TSharedPtr<SFilterSearchBox>, FilterSearchBox)

		/** The filter menu name. Will register a menu using this name, if not already registered. */
		SLATE_ARGUMENT(FName, FilterMenuName)
	
		/** A unique identifier for this filter bar needed to enable saving settings in a config file */
		SLATE_ARGUMENT(FName, FilterBarIdentifier)
	
		/** Whether the filter bar should provide the default asset filters */
		SLATE_ARGUMENT(bool, UseDefaultAssetFilters)

		/** Called when the filter bar layout is changing, before the filters are added to the layout */
		SLATE_EVENT(SBasicFilterBar<FilterType>::FOnFilterBarLayoutChanging, OnFilterBarLayoutChanging)
		
		/** The layout that determines how the filters are laid out */
		SLATE_ARGUMENT(EFilterBarLayout, FilterBarLayout)
		
		/** If true, allow dynamically changing the orientation and saving in the config */
		SLATE_ARGUMENT(bool, CanChangeOrientation)

		/** Determines how each individual filter pill looks like */
		SLATE_ARGUMENT(EFilterPillStyle, FilterPillStyle)

		/** Expands the specified asset category, if specified. If not, it will expand Basic/Common instead. */
		SLATE_ARGUMENT(TOptional<FAssetCategoryPath>, DefaultMenuExpansionCategory)

		/** If true, adds custom categories as sections (expanded) vs. as sub-menus */
		SLATE_ARGUMENT(bool, bUseSectionsForCustomCategories)

		/** The command list provided to the filter context menu */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
	
	SLATE_END_ARGS()

	virtual ~SAssetFilterBar() override;

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs )
	{
		bUseDefaultAssetFilters = InArgs._UseDefaultAssetFilters;
		CustomClassFilters = InArgs._CustomClassFilters;
		UserCustomClassFilters = InArgs._CustomClassFilters;
		FilterMenuName = InArgs._FilterMenuName;
		FilterBarIdentifier = InArgs._FilterBarIdentifier;
		DefaultMenuExpansionCategory = InArgs._DefaultMenuExpansionCategory;
		CommandListWeak = InArgs._CommandList;
		
		typename SBasicFilterBar<FilterType>::FArguments Args;
		Args._OnFilterChanged = InArgs._OnFilterChanged;
		Args._CustomFilters = InArgs._CustomFilters;
		Args._OnExtendAddFilterMenu = InArgs._OnExtendAddFilterMenu;
		Args._CreateTextFilter = InArgs._CreateTextFilter;
		Args._FilterSearchBox = InArgs._FilterSearchBox;
		Args._OnFilterBarLayoutChanging = InArgs._OnFilterBarLayoutChanging;
		Args._FilterBarLayout = InArgs._FilterBarLayout;
		Args._CanChangeOrientation = InArgs._CanChangeOrientation;
		Args._FilterPillStyle = InArgs._FilterPillStyle;
		Args._UseSectionsForCategories = InArgs._bUseSectionsForCustomCategories;
		
		SBasicFilterBar<FilterType>::Construct(Args);

		InitializeConfig();
		
		// Create the Asset Type Action filters if requested
		CreateAssetTypeActionFilters();

		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
		const TSharedRef<FPathPermissionList>& AssetClassPermissionList = AssetToolsModule.Get().GetAssetClassPathPermissionList(AssetClassAction);

		// Re-create the Asset Type Action filters whenever the permission list changes
		AssetClassPermissionList->OnFilterChanged().AddSP(this, &SAssetFilterBar<FilterType>::CreateAssetTypeActionFilters);
		AssetRegistryVersionChangeHandle = UAssetDefinitionRegistry::Get()->OnAssetDefinitionRegistryVersionChange().AddLambda([this](UAssetDefinitionRegistry*) { CreateAssetTypeActionFilters(); });
	}

	virtual void SaveSettings()
	{
		FFilterBarSettings* FilterBarConfig = GetMutableConfig();

		if(!FilterBarConfig)
		{
			UE_LOGF(LogSlate, Error, "SFilterBar Requires that you specify a FilterBarIdentifier to save settings");
			return;
		}

		// Empty the config, we are just going to re-save everything
		FilterBarConfig->Empty();

		SaveSettingsInternal(FilterBarConfig);
	}

	virtual void LoadSettings()
	{
		const FFilterBarSettings* FilterBarConfig = GetConstConfig();
		
		if(!FilterBarConfig)
		{
			UE_LOGF(LogSlate, Error, "SFilterBar Requires that you specify a FilterBarIdentifier to load settings");
			return;
		}

		if(!FilterBarConfig->CustomTextFilters.IsEmpty())
		{
			// We must have a CreateTextFilter bound if we have any custom text filters saved!!
			check(this->CreateTextFilter.IsBound());
		}

		LoadFilters(FilterBarConfig);
		LoadCustomTextFilters(FilterBarConfig);

		this->OnFilterChanged.ExecuteIfBound();
	}

protected:

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if(MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
		{
			FReply Reply = FReply::Handled().ReleaseMouseCapture();
			
			TSharedPtr<SWidget> MenuContent = MakeAddFilterMenu();

			if(MenuContent.IsValid())
			{
				FVector2D SummonLocation = MouseEvent.GetScreenSpacePosition();
				FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
				FSlateApplication::Get().PushMenu(this->AsShared(), WidgetPath, MenuContent.ToSharedRef(), SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
			}

			return Reply;
		}

		return FReply::Unhandled();
	}
	
	/** Save all the custom class filters (created by the user) into the specified config */
	void SaveCustomTextFilters(FFilterBarSettings* FilterBarConfig)
	{
		/* Go through all the custom text filters (including unchecked ones) to save them, so that the user does not
		 * lose the text filters they created from the menu or a saved search
		 */
		for (const TSharedRef<ICustomTextFilter<FilterType>>& CustomTextFilter : this->CustomTextFilters)
		{
			// Get the actual FFilterBase
			TSharedRef<FFilterBase<FilterType>> CustomFilter = CustomTextFilter->GetFilter().ToSharedRef();
			
			// Is the filter "checked", i.e visible in the filter bar
			bool bIsChecked = this->IsFrontendFilterInUse(CustomFilter);

			// Is the filter "active", i.e visible and enabled in the filter bar
			bool bIsActive = this->IsFilterActive(CustomFilter);

			// Get the data associated with this filter
			FCustomTextFilterData FilterData = CustomTextFilter->CreateCustomTextFilterData();

			FCustomTextFilterState FilterState;
			FilterState.bIsChecked = bIsChecked;
			FilterState.bIsActive = bIsActive;
			FilterState.FilterData = FilterData;

			FilterBarConfig->CustomTextFilters.Add(FilterState);
		}
	}

	virtual void SaveSettingsInternal(FFilterBarSettings* InSettingsToSave)
	{
		check(InSettingsToSave);

		SaveCustomTextFilters(InSettingsToSave);
		SaveFilters(InSettingsToSave);

		SaveConfig();
	}

	/** Save all the custom filters (created programmatically) into the specified config */
	void SaveFilters(FFilterBarSettings* FilterBarConfig)
	{
		/* For the remaining (custom and type) filters, go through the currently active (i.e visible in the filter bar)
		 * ones to save their state, since they will be added to the filter bar programmatically every time
		 */
		for ( const TSharedPtr<SFilter> Filter : this->Filters )
		{
			const FString FilterName = Filter->GetFilterName();

			// Ignore custom text filters, since we saved them previously
			if(FilterName == FCustomTextFilter<FilterType>::GetFilterTypeName().ToString())
			{
				continue;
			}
			// If it is a FrontendFilter
			else if ( Filter->GetFrontendFilter().IsValid() )
			{
				FilterBarConfig->CustomFilters.Add(FilterName, Filter->IsEnabled());
			}
			// Otherwise we assume it is a type filter
			else
			{
				FilterBarConfig->TypeFilters.Add(FilterName, Filter->IsEnabled());
			}
		}

		// Add back all the unknown filters which we encountered while loading, but didn't find in our instance
		FilterBarConfig->TypeFilters.Append(UnknownTypeFilters);
	}

	/** Load all the custom class filters (created by the user) from the specified config */
	void LoadCustomTextFilters(const FFilterBarSettings* FilterBarConfig)
	{
		// Load all the custom text filters
		for(const FCustomTextFilterState& FilterState : FilterBarConfig->CustomTextFilters)
		{
			// Create an ICustomTextFilter using the provided delegate
			TSharedRef<ICustomTextFilter<FilterType>> NewTextFilter = this->CreateTextFilter.Execute().ToSharedRef();

			// Get the actual FFilterBase
			TSharedRef<FFilterBase<FilterType>> NewFilter = NewTextFilter->GetFilter().ToSharedRef();

			// Set the internals of the custom text filter from what we have saved
			NewTextFilter->SetFromCustomTextFilterData(FilterState.FilterData);

			// Add this to our list of custom text filters
			this->CustomTextFilters.Add(NewTextFilter);

			// If the filter was checked previously, add it to the filter bar
			if(FilterState.bIsChecked)
			{
				TSharedRef<SFilter> AddedFilter = this->AddFilterToBar(NewFilter);

				// Set the filter as active if it was previously
				AddedFilter->SetEnabled(FilterState.bIsActive, false);
				
				this->SetFrontendFilterActive(NewFilter, FilterState.bIsActive);
			}
		}
	}
	
	/** Load all the custom filters (created programmatically) from the specified config */
	void LoadFilters(const FFilterBarSettings* FilterBarConfig)
	{
		// Load all the custom filters (i.e FrontendFilters)
		TMap<FString, TSharedRef<FFilterBase<FilterType>>> CustomFilters;
		CustomFilters.Reserve(FilterBarConfig->CustomFilters.Num());
		for (const TSharedRef<FFilterBase<FilterType>>& FrontendFilter : this->AllFrontendFilters)
		{
			if (!this->IsFrontendFilterInUse(FrontendFilter))
			{
				// Try to find this type filter in our list of saved filters
				const FString& FilterName = FrontendFilter->GetName();
				if (FilterBarConfig->CustomFilters.Contains(FilterName))
				{
					CustomFilters.Add(FilterName, FrontendFilter);
				}
			}
		}

		// We add them here to preserve the saving config order
		for (const TPair<FString, bool>& CustomFilter : FilterBarConfig->CustomFilters)
		{
			if (const TSharedRef<FFilterBase<FilterType>>* Filter = CustomFilters.Find(CustomFilter.Key))
			{
				TSharedRef<SFilter> NewFilter = this->AddFilterToBar(*Filter);
				NewFilter->SetEnabled(CustomFilter.Value, false);
				this->SetFrontendFilterActive(*Filter, NewFilter->IsEnabled());
			}
		}

		// Save a copy of all the type filters, and then remove anything we find
		UnknownTypeFilters = FilterBarConfig->TypeFilters;

		// Load all the type filters
		TMap<FString, TSharedRef<FCustomClassFilterData>> AssetClassFilters;
		AssetClassFilters.Reserve(FilterBarConfig->TypeFilters.Num());
		for(const TSharedRef<FCustomClassFilterData> &CustomClassFilter : this->CustomClassFilters)
		{
			const FString FilterName = CustomClassFilter->GetFilterName();
			
			if(!this->IsClassTypeInUse(CustomClassFilter))
			{
				// Try to find this type filter in our list of saved filters
				if (FilterBarConfig->TypeFilters.Contains(FilterName))
				{
					AssetClassFilters.Add(FilterName, CustomClassFilter);

					// Remove it from the Unknown list, we have encountered this type filter in the current instance
					UnknownTypeFilters.Remove(FilterName);
				}
			}
			else
			{
				// If the filter is already in use, we don't want to add a duplicate but it isn't unknown - so remove it from the UnknownTypeFilters
				UnknownTypeFilters.Remove(FilterName);
			}
		}

		// We add them here to preserve the saving config order
		for (const TPair<FString, bool>& TypeFilter : FilterBarConfig->TypeFilters)
		{
			if (const TSharedRef<FCustomClassFilterData>* Filter = AssetClassFilters.Find(TypeFilter.Key))
			{
				TSharedRef<SFilter> NewFilter = this->AddAssetFilterToBar(*Filter);
				NewFilter->SetEnabled(TypeFilter.Value, false);
			}
		}
	}
	
	typedef typename SBasicFilterBar<FilterType>::SFilter SFilter;

	/** A subclass of SFilter in SBasicFilterBar to add functionality for Asset Filters */
	class SAssetFilter : public SFilter
	{
		using FOnRequestRemove = typename SBasicFilterBar<FilterType>::SFilter::FOnRequestRemove;
		using FOnRequestEnableOnly = typename SBasicFilterBar<FilterType>::SFilter::FOnRequestEnableOnly;
		using FOnRequestEnableAll = typename SBasicFilterBar<FilterType>::SFilter::FOnRequestEnableAll;
		using FOnRequestDisableAll = typename SBasicFilterBar<FilterType>::SFilter::FOnRequestDisableAll;
		using FOnRequestRemoveAll = typename SBasicFilterBar<FilterType>::SFilter::FOnRequestRemoveAll;
		using FOnRequestRemoveAllButThis = typename SBasicFilterBar<FilterType>::SFilter::FOnRequestRemoveAllButThis;
		
		SLATE_BEGIN_ARGS( SAssetFilter ){}

			/** The Custom Class Data that is associated with this filter */
			SLATE_ARGUMENT( TSharedPtr<FCustomClassFilterData>, CustomClassFilter )

			// SFilter Arguments
		
			/** If this is an front end filter, this is the filter object */
			SLATE_ARGUMENT( TSharedPtr<FFilterBase<FilterType>>, FrontendFilter )

			/** Invoked when the filter toggled */
			SLATE_EVENT( FOnFilterChanged, OnFilterChanged )

			/** Invoked when a request to remove this filter originated from within this filter */
			SLATE_EVENT( FOnRequestRemove, OnRequestRemove )

			/** Invoked when a request to enable only this filter originated from within this filter */
			SLATE_EVENT( FOnRequestEnableOnly, OnRequestEnableOnly )

			/** Invoked when a request to enable all filters originated from within this filter */
			SLATE_EVENT( FOnRequestEnableAll, OnRequestEnableAll)

			/** Invoked when a request to disable all filters originated from within this filter */
			SLATE_EVENT( FOnRequestDisableAll, OnRequestDisableAll )

			/** Invoked when a request to remove all filters originated from within this filter */
			SLATE_EVENT( FOnRequestRemoveAll, OnRequestRemoveAll )

			/** Invoked when a request to remove all filters originated from within this filter */
			SLATE_EVENT( FOnRequestRemoveAllButThis, OnRequestRemoveAllButThis )

			/** Determines how each individual filter pill looks like */
			SLATE_ARGUMENT(EFilterPillStyle, FilterPillStyle)

		SLATE_END_ARGS()

		/** Constructs this widget with InArgs */
		void Construct( const FArguments& InArgs )
		{
			this->bEnabled = false;
			this->OnFilterChanged = InArgs._OnFilterChanged;
			this->OnRequestRemove = InArgs._OnRequestRemove;
			this->OnRequestEnableOnly = InArgs._OnRequestEnableOnly;
			this->OnRequestEnableAll = InArgs._OnRequestEnableAll;
			this->OnRequestDisableAll = InArgs._OnRequestDisableAll;
			this->OnRequestRemoveAll = InArgs._OnRequestRemoveAll;
			this->OnRequestRemoveAllButThis = InArgs._OnRequestRemoveAllButThis;
			this->FrontendFilter = InArgs._FrontendFilter;

			CustomClassFilter = InArgs._CustomClassFilter;
			
			// Get the tooltip and color of the type represented by this filter
			this->FilterColor = FLinearColor::White;
			if ( CustomClassFilter.IsValid() )
			{
				this->FilterColor = CustomClassFilter->GetColor();
				// No tooltip for asset type filters
			}
			else if ( this->FrontendFilter.IsValid() )
			{
				this->FilterColor = this->FrontendFilter->GetColor();
				this->FilterToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this->FrontendFilter.ToSharedRef(), &FFilterBase<FilterType>::GetToolTipText));
			}

			SFilter::Construct_Internal(InArgs._FilterPillStyle);
		}

	public:
		
		/** Returns this widgets contribution to the combined filter */
		FARFilter GetBackendFilter() const
		{
			FARFilter Filter;

			if(CustomClassFilter)
			{
				CustomClassFilter->BuildBackendFilter(Filter);
			}

			return Filter;
		}
		
		/** Gets the asset type actions associated with this filter */
		const TSharedPtr<FCustomClassFilterData>& GetCustomClassFilterData() const
		{
			return CustomClassFilter;
		}

		void SetCustomClassFilterData(const TSharedRef<FCustomClassFilterData>& InCustomClassFilterData)
		{
			CustomClassFilter = InCustomClassFilterData;
		}

		/** Returns the display name for this filter */
		virtual FText GetFilterDisplayName() const override
		{
			if (CustomClassFilter.IsValid())
			{
				return CustomClassFilter->GetName();
			}
			else
			{
				return SFilter::GetFilterDisplayName();
			}

		}

		virtual FString GetFilterName() const override
		{
			if (CustomClassFilter.IsValid())
			{
				return CustomClassFilter->GetFilterName();
			}
			else
			{
				return SFilter::GetFilterName();
			}
		}
		
	protected:
		/** The asset type actions that are associated with this filter */
		TSharedPtr<FCustomClassFilterData> CustomClassFilter;
	};

public:

	/** Use this function to get an FARFilter that represents all the AssetType filters that are currently active */
	FARFilter GetCombinedBackendFilter() const
	{
		FARFilter CombinedFilter;

		// Add all selected filters
		for (int32 FilterIdx = 0; FilterIdx < AssetFilters.Num(); ++FilterIdx)
		{
			const TSharedPtr<SAssetFilter> AssetFilter = AssetFilters[FilterIdx];
			
			if ( AssetFilter.IsValid() && AssetFilter->IsEnabled())
			{
				CombinedFilter.Append(AssetFilter->GetBackendFilter());
			}
		}

		// HACK: A blueprint can be shown as Blueprint or as BlueprintGeneratedClass, but we don't want to distinguish them while filtering.
		// This should be removed, once all blueprints are shown as BlueprintGeneratedClass.
		// Note: Adding the BlueprintGeneratedClass should occur before the below bRecursiveClasses check otherwise it will be added to the ExclusionSet
		if (CombinedFilter.ClassPaths.Contains(FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("Blueprint"))))
		{
			CombinedFilter.ClassPaths.AddUnique(FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("BlueprintGeneratedClass")));
		}

		if ( CombinedFilter.bRecursiveClasses )
		{
			// Add exclusions for AssetTypeActions NOT in the filter.
			// This will prevent assets from showing up that are both derived from an asset in the filter set and derived from an asset not in the filter set
			// Get the list of all asset type actions
			for(const TSharedRef<FCustomClassFilterData> &CustomClassFilter : CustomClassFilters)
			{
				const UClass* TypeClass = CustomClassFilter->GetClass();
				if (TypeClass && !CombinedFilter.ClassPaths.Contains(TypeClass->GetClassPathName()))
				{
					CombinedFilter.RecursiveClassPathsExclusionSet.Add(TypeClass->GetClassPathName());
				}
			}
		}

		return CombinedFilter;
	}

	/** Check if there is a filter associated with the given class represented by FTopLevelAssetPath in the filter bar
	 * @param	InClassPath		The Class the filter is associated with
	 */
	bool DoesAssetTypeFilterExist(const FTopLevelAssetPath& InClassPath)
	{
		for(const TSharedRef<FCustomClassFilterData>& CustomClassFilterData : CustomClassFilters)
		{
			if(CustomClassFilterData->GetClassPathName() == InClassPath)
			{
				return true;
			}
		}

		return false;
	}

	/** Set the check box state of the specified filter (in the filter drop down) and pin/unpin a filter widget on/from the filter bar. When a filter is pinned (was not already pinned), it is activated if requested and deactivated when unpinned.
	 * @param	InClassPath		The Class the filter is associated with (must exist in the widget)
	 * @param	InCheckState	The CheckState to apply to the flter
	 * @param	bEnableFilter	Whether the filter should be enabled when it is pinned
	 */
	void SetAssetTypeFilterCheckState(const FTopLevelAssetPath& InClassPath, ECheckBoxState InCheckState, bool bEnableFilter = true)
	{
		for(const TSharedRef<FCustomClassFilterData>& CustomClassFilterData : CustomClassFilters)
		{
			if(CustomClassFilterData->GetClassPathName() == InClassPath)
			{
				bool FilterChecked = IsClassTypeInUse(CustomClassFilterData);

				if (InCheckState == ECheckBoxState::Checked && !FilterChecked)
				{
					TSharedRef<SFilter> NewFilter = AddAssetFilterToBar(CustomClassFilterData);

					if(bEnableFilter)
					{
						NewFilter->SetEnabled(true);
					}
				}
				else if (InCheckState == ECheckBoxState::Unchecked && FilterChecked)
				{
					RemoveAssetFilter(CustomClassFilterData); // Unpin the filter widget and deactivate the filter.
				}
				// else -> Already in the desired 'check' state.
				
			}
		}
	}

	/** Returns the check box state of the specified filter (in the filter drop down). This tells whether the filter is pinned or not on the filter bar, but not if filter is active or not.
	 * @see		IsFilterActive().
	 * @param	InClassPath		The Class the filter is associated with
	 * @return The CheckState if a filter associated with the class name is in the filter bar, ECheckBoxState::Undetermined otherwise
	 */
	
	ECheckBoxState GetAssetTypeFilterCheckState(const FTopLevelAssetPath& InClassPath) const
	{
		for(const TSharedRef<FCustomClassFilterData>& CustomClassFilterData : CustomClassFilters)
		{
			if(CustomClassFilterData->GetClassPathName() == InClassPath)
			{
				return IsClassTypeInUse(CustomClassFilterData) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; 
			}
		}

		return ECheckBoxState::Undetermined;
	}

	/** Returns true if the specified filter is both checked (pinned on the filter bar) and active (contributing to filter the result).
	 *  @param	InClassPath		The Class the filter is associated with
	 */
	bool IsAssetTypeFilterActive(const FTopLevelAssetPath& InClassPath) const
	{
		for(const TSharedPtr<SAssetFilter> AssetFilter : AssetFilters)
		{
			if(AssetFilter->GetCustomClassFilterData()->GetClassPathName() == InClassPath)
			{
				return AssetFilter->IsEnabled();
			}
		}
		
		return false;
	}

	/** If a filter with the input class name is Checked (i.e visible in the bar), enable/disable it
	 * @see SetFilterCheckState to set the check state and GetFilterCheckState to check if it is checked
	 * @param	InClassPath		The Class the filter is associated with
	 * @param	bEnable			Whether to enable or disable the filter
	 */
	void ToggleAssetTypeFilterEnabled(const FTopLevelAssetPath& InClassPath, bool bEnable)
	{
		for(const TSharedPtr<SAssetFilter> AssetFilter : AssetFilters)
		{
			if(AssetFilter->GetCustomClassFilterData()->GetClassPathName() == InClassPath)
			{
				AssetFilter->SetEnabled(bEnable);
				return;
			}
		}
	}

	/** Remove all filters from the filter bar, while disabling any active ones */
	virtual void RemoveAllFilters() override
	{
		AssetFilters.Empty();
		SBasicFilterBar<FilterType>::RemoveAllFilters();
	}

	/** Handler for when filter by type is selected */
	void FilterByType(TSharedPtr<FCustomClassFilterData> CustomClassFilterData)
	{
		if (CustomClassFilterData.IsValid())
		{
			if (IsClassTypeInUse(CustomClassFilterData))
			{
				RemoveAssetFilter(CustomClassFilterData);
			}
			else
			{
				TSharedRef<SFilter> NewFilter = AddAssetFilterToBar(CustomClassFilterData);
				NewFilter->SetEnabled(true);
			}
		}
	}

	/** Handler for when filter by type category is selected */
	void FilterByTypeCategory(TSharedPtr<FFilterCategory> TypeCategory, TArray<TSharedPtr<FCustomClassFilterData>> Classes)
	{
		bool bFullCategoryInUse = IsTypeCategoryInUse(TypeCategory, Classes);
		bool ExecuteOnFilterChanged = false;

		for (const TSharedPtr<FCustomClassFilterData>& CustomClass : Classes)
		{
			if (bFullCategoryInUse)
			{
				RemoveAssetFilter(CustomClass);
				ExecuteOnFilterChanged = true;
			}
			else if (!IsClassTypeInUse(CustomClass))
			{
				TSharedRef<SFilter> NewFilter = AddAssetFilterToBar(CustomClass);
				NewFilter->SetEnabled(true, false);
				ExecuteOnFilterChanged = true;
			}
		}

		if (ExecuteOnFilterChanged)
		{
			this->OnFilterChanged.ExecuteIfBound();
		}
	}

	/** Set the active asset filters for this filter list */
	void SetActiveFilters(TConstArrayView<FTopLevelAssetPath> InFilters)
	{
		RemoveAllFilters();
		for (const FTopLevelAssetPath& FilterAssetPath : InFilters)
		{
			SetAssetTypeFilterCheckState(FilterAssetPath, ECheckBoxState::Checked, /** Enabled */true);
		}
	}

protected:

	/** AssetFilter specific override to SBasicFilterBar::RemoveAllButThis */
	virtual void RemoveAllButThis(const TSharedRef<SFilter>& FilterToKeep) override
	{
		TSharedPtr<SAssetFilter> AssetFilterToKeep;

		// Make sure to keep it in our local list of AssetFilters
		for(const TSharedPtr<SAssetFilter> AssetFilter : AssetFilters)
		{
			if(AssetFilter == FilterToKeep)
			{
				AssetFilterToKeep = AssetFilter;
			}
		}

		AssetFilters.Empty();

		if(AssetFilterToKeep)
		{
			AssetFilters.Add(AssetFilterToKeep.ToSharedRef());
		}

		SBasicFilterBar<FilterType>::RemoveAllButThis(FilterToKeep);
	}

	/** Add an AssetFilter to the toolbar, making it "Active" but not enabled */
	TSharedRef<SFilter> AddAssetFilterToBar(const TSharedPtr<FCustomClassFilterData>& CustomClassFilter)
	{
		TSharedRef<SAssetFilter> NewFilter =
			SNew(SAssetFilter)
			.FilterPillStyle(this->FilterPillStyle)
			.CustomClassFilter(CustomClassFilter)
			.OnFilterChanged(this->OnFilterChanged)
			.OnRequestRemove(this, &SAssetFilterBar<FilterType>::RemoveFilterAndUpdate)
			.OnRequestEnableOnly(this, &SAssetFilterBar<FilterType>::EnableOnlyThisFilter)
			.OnRequestEnableAll(this, &SAssetFilterBar<FilterType>::EnableAllFilters)
			.OnRequestDisableAll(this, &SAssetFilterBar<FilterType>::DisableAllFilters)
			.OnRequestRemoveAll(this, &SAssetFilterBar<FilterType>::RemoveAllFilters)
			.OnRequestRemoveAllButThis(this, &SAssetFilterBar<FilterType>::RemoveAllButThis);

		this->AddFilterToBar( NewFilter );

		// Add it to our list of just AssetFilters
		AssetFilters.Add(NewFilter);
		
		return NewFilter;
	}
	
	/** Remove a filter from the filter bar */
	virtual void RemoveFilter(const TSharedRef<SFilter>& FilterToRemove) override
	{
		SBasicFilterBar<FilterType>::RemoveFilter(FilterToRemove);

		AssetFilters.RemoveAll([&FilterToRemove](TSharedRef<SAssetFilter>& AssetFilter) { return AssetFilter == FilterToRemove; });
	}

	/** Handler for when the remove filter button was clicked on a filter */
	void RemoveAssetFilter(const TSharedPtr<FCustomClassFilterData>& CustomClassData, bool ExecuteOnFilterChanged = true)
	{
		TSharedPtr<SAssetFilter> FilterToRemove;
		for ( const TSharedPtr<SAssetFilter> AssetFilter : AssetFilters )
		{
			const TSharedPtr<FCustomClassFilterData>& ClassData = AssetFilter->GetCustomClassFilterData();
			if (ClassData == CustomClassData)
			{
				FilterToRemove = AssetFilter;
				break;
			}
		}

		if ( FilterToRemove.IsValid() )
		{
			if (ExecuteOnFilterChanged)
			{
				this->RemoveFilterAndUpdate(FilterToRemove.ToSharedRef());
			}
			else
			{
				this->RemoveFilter(FilterToRemove.ToSharedRef());
			}

			// Remove it from our local list of AssetFilters
			AssetFilters.Remove(FilterToRemove.ToSharedRef());
		}
	}

	/** Create the default set of IAssetTypeActions Filters provided with the widget if requested */
	void CreateAssetTypeActionFilters()
	{
		if(!bUseDefaultAssetFilters)
		{
			return;
		}

		// Empty the existing categories and filters, we will re-add them based on the permission list
		AssetFilterCategories.Empty();
		CustomClassFilters.Empty();

		// Re-add the CustomClassFilters added by the user manually, these don't get tested against the permission list
		CustomClassFilters.Append(UserCustomClassFilters);

		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));

		{
			// Add the Basic category
			TSharedPtr<FFilterCategory> BasicCategory = MakeShareable(new FFilterCategory(LOCTEXT("BasicFilter", "Common"), LOCTEXT("BasicFilterTooltip", "Filter by Common assets.")));
			AssetFilterCategories.Add(EAssetCategoryPaths::Basic.GetCategory(), BasicCategory);
		}

		{
			TArray<TObjectPtr<UAssetDefinition>> AssetDefinitions = UAssetDefinitionRegistry::Get()->GetAllAssetDefinitions();

			const TSharedRef<FPathPermissionList>& AssetClassPermissionList = AssetToolsModule.Get().GetAssetClassPathPermissionList(AssetClassAction);

			// For every asset type, convert it to an FCustomClassFilterData and add it to the list
			for (UAssetDefinition* AssetDefinition : AssetDefinitions)
			{
				TSoftClassPtr<UObject> AssetClass = AssetDefinition->GetAssetClass();
				if ((!AssetClass.IsNull() && AssetClassPermissionList->PassesFilter(AssetClass.ToString())))
				{
					const TSharedRef<FAssetFilterDataCache> FilterCache = AssetDefinition->GetFilters();
				
					for (const FAssetFilterData& FilterData : FilterCache->Filters)
					{
						ensureMsgf(FilterData.FilterCategories.Num() > 0, TEXT("%s is missing Filter Categories, without any filter categories we can't display this filter."), *FilterData.Name);
					
						// Convert the AssetTypeAction to an FCustomClassFilterData and add it to our list
						TSharedRef<FCustomClassFilterData> CustomClassFilterData = MakeShared<FCustomClassFilterData>(AssetDefinition, FilterData);

						for (const FAssetCategoryPath& CategoryPath : FilterData.FilterCategories)
						{
							TSharedPtr<FFilterCategory> FilterCategory = AssetFilterCategories.FindRef(CategoryPath.GetCategory());
							if (!FilterCategory.IsValid())
							{
								const FText Tooltip = FText::Format(LOCTEXT("WildcardFilterTooltip", "Filter by {0} Assets."), CategoryPath.GetCategoryText());
								FilterCategory = MakeShared<FFilterCategory>(CategoryPath.GetCategoryText(), Tooltip);
								AssetFilterCategories.Add(CategoryPath.GetCategory(), FilterCategory);
							}

							CustomClassFilterData->AddCategory(FilterCategory);
						}
						
						CustomClassFilters.Add(CustomClassFilterData);
					}
				}
			}
		}

		// Update/remove any asset filters that already exist and use AssetTypeActions
		TArray<TSharedRef<SAssetFilter>> AssetFiltersToRemove;
		
		for (TSharedRef<SAssetFilter>& AssetFilter : AssetFilters)
		{
			// Try and find the FCustomClassFilterData for the current filter using the new list we just created in CustomClassFilters
			TSharedRef<FCustomClassFilterData>* FoundCustomClassFilterData = CustomClassFilters.FindByPredicate([AssetFilter](const TSharedRef<FCustomClassFilterData>& CustomClassFilterData)
			{
				return CustomClassFilterData->GetFilterName() == AssetFilter->GetCustomClassFilterData()->GetFilterName();
			});

			// If it exists, update the filter to let it know
			if (FoundCustomClassFilterData)
			{
				AssetFilter->SetCustomClassFilterData(*FoundCustomClassFilterData);
			}
			// Otherwise this filter had an AssetTypeAction that doesn't pass the new permission list, remove it
			else
			{
				AssetFiltersToRemove.Add(AssetFilter);
			}
		}

		for (TSharedRef<SAssetFilter>& AssetFilter : AssetFiltersToRemove)
		{
			RemoveFilter(AssetFilter);
		}
	}
	
	/* Filter Dropdown Related Functionality */

	/** Handler for when the add filter menu is populated by a category */
	void CreateFiltersMenuCategory(FToolMenuSection& Section, const TArray<TSharedPtr<FCustomClassFilterData>> CustomClassFilterDatas) const
	{
		for (int32 ClassIdx = 0; ClassIdx < CustomClassFilterDatas.Num(); ++ClassIdx)
		{
			const TSharedPtr<FCustomClassFilterData>& CustomClassFilterData = CustomClassFilterDatas[ClassIdx];

			const FText& LabelText = CustomClassFilterData->GetName();
			Section.AddMenuEntry(
				NAME_None,
				LabelText,
				FText::Format( LOCTEXT("FilterByTooltipPrefix", "Filter by {0}"), LabelText ),
				FSlateIconFinder::FindIconForClass(CustomClassFilterData->GetClass()),
				FUIAction(
					FExecuteAction::CreateSP( const_cast< SAssetFilterBar<FilterType>* >(this), &SAssetFilterBar<FilterType>::FilterByType, CustomClassFilterData ),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SAssetFilterBar<FilterType>::IsClassTypeInUse, CustomClassFilterData ) ),
				EUserInterfaceActionType::ToggleButton
				);
		}
	}

	void CreateFiltersMenuCategory(UToolMenu* InMenu, const TArray<TSharedPtr<FCustomClassFilterData>> CustomClassFilterDatas) const
	{
		CreateFiltersMenuCategory(InMenu->AddSection("Section"), CustomClassFilterDatas);
	}

	/** Override this if you want to change aspects on the add filter menu.
	 *  You can opt to only override the OnFilterAssetType member to exclude assets, or choose to override the entire populate callback
	 */
	virtual UAssetFilterBarContext* CreateAssetFilterBarContext()
	{
		UAssetFilterBarContext* FilterBarContext = NewObject<UAssetFilterBarContext>();
		FilterBarContext->PopulateFilterMenu = FOnPopulateAddAssetFilterMenu::CreateSP(this, &SAssetFilterBar<FilterType>::PopulateAddFilterMenu);
		FilterBarContext->OnExtendAddFilterMenu = this->OnExtendAddFilterMenu;
		
		if(DefaultMenuExpansionCategory.IsSet() && AssetFilterCategories.Contains(DefaultMenuExpansionCategory->GetCategory()))
		{
			FilterBarContext->MenuExpansion = AssetFilterCategories[DefaultMenuExpansionCategory->GetCategory()];
		};

		return FilterBarContext;
	}
	
	/** Handler for when the add filter button was clicked */
	virtual TSharedRef<SWidget> MakeAddFilterMenu() override final
	{
		if (!UToolMenus::Get()->IsMenuRegistered(FilterMenuName))
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(FilterMenuName);
			Menu->bShouldCloseWindowAfterMenuSelection = true;
			Menu->bCloseSelfOnly = true;

			Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				if (UAssetFilterBarContext* Context = InMenu->FindContext<UAssetFilterBarContext>())
				{
					Context->PopulateFilterMenu.ExecuteIfBound(InMenu, Context->MenuExpansion, Context->OnFilterAssetType);
					Context->OnExtendAddFilterMenu.ExecuteIfBound(InMenu);
				}
			}));
		}

		const FToolMenuContext ToolMenuContext(CommandListWeak.Pin(), TSharedPtr<FExtender>(), CreateAssetFilterBarContext());
		return UToolMenus::Get()->GenerateWidget(FilterMenuName, ToolMenuContext);
	} 

	/** Handler to Populate the Add Filter Menu. Use OnFilterAssetType in Subclasses to add classes to the exclusion list */
	virtual void PopulateAddFilterMenu(UToolMenu* Menu, TSharedPtr<FFilterCategory> MenuExpansion, FOnFilterAssetType OnFilterAssetType)
	{
		using namespace UE::Editor::Widgets;

		TMap<TSharedPtr<FFilterCategory>, FFilterCategoryMenu> CategoryToMenuMap = BuildCategoryToMenuMap(AssetFilterCategories, CustomClassFilters, OnFilterAssetType);

		// Populate the common filter sections (Reset Filters etc)
		{
			this->PopulateCommonFilterSections(Menu);
		}

		// If we want to expand a category
		if (MenuExpansion)
		{
			// First add the expanded category, this appears as standard entries in the list (Note: intentionally not using FindChecked here as removing it from the map later would cause the ref to be garbage)
			if (FFilterCategoryMenu* ExpandedCategory = CategoryToMenuMap.Find( MenuExpansion ))
			{
				FToolMenuSection& Section = Menu->AddSection(ExpandedCategory->SectionExtensionHook, ExpandedCategory->SectionHeading);

				// If we are doing a full menu (i.e expanding basic) we add a menu entry which toggles all other categories
                Section.AddMenuEntry(
                	FName(FText::AsCultureInvariant(ExpandedCategory->SectionHeading).ToString()),
                	ExpandedCategory->SectionHeading,
                	MenuExpansion->Tooltip,
                	FSlateIcon(FAppStyle::Get().GetStyleSetName(), "PlacementBrowser.Icons.Basic"),
                	FUIAction(
                	FExecuteAction::CreateSP( this, &SAssetFilterBar<FilterType>::FilterByTypeCategory, MenuExpansion, ExpandedCategory->Classes ),
                	FCanExecuteAction(),
                	FGetActionCheckState::CreateSP(this, &SAssetFilterBar<FilterType>::IsTypeCategoryChecked, MenuExpansion, ExpandedCategory->Classes ) ),
                	EUserInterfaceActionType::ToggleButton
                	);

				Section.AddSeparator("ExpandedCategorySeparator");

				// Now populate with all the assets from the expanded category
				SAssetFilterBar<FilterType>::CreateFiltersMenuCategory( Section, ExpandedCategory->Classes);

				// Remove the Expanded from the map now, as this is treated differently and is no longer needed.
				CategoryToMenuMap.Remove(MenuExpansion);
			}
		}

		// Add all the other categories as submenus
		FToolMenuSection& Section = Menu->AddSection("AssetFilterBarFilterAdvancedAsset", LOCTEXT("AdvancedAssetsMenuHeading", "Type Filters"));

		// Sort by category name so that we add the submenus in alphabetical order
		CategoryToMenuMap.KeySort([](const TSharedPtr<FFilterCategory>& A, const TSharedPtr<FFilterCategory>& B) {
			return A->Title.CompareTo(B->Title) < 0;
		});

		// Sort each submenu internals 
		for (TPair<TSharedPtr<FFilterCategory>, FFilterCategoryMenu>& CategoryToMenu : CategoryToMenuMap)
		{
			CategoryToMenu.Value.Classes.Sort([](const TSharedPtr<FCustomClassFilterData>& A, const TSharedPtr<FCustomClassFilterData>& B)
			{
				return A->GetName().CompareTo(B->GetName()) < 0;
			});
		}

		// Now actually add them
		for (const TPair<TSharedPtr<FFilterCategory>, FFilterCategoryMenu>& CategoryMenuPair : CategoryToMenuMap)
		{
			Section.AddSubMenu(
				FName(FText::AsCultureInvariant(CategoryMenuPair.Key->Title).ToString()),
				CategoryMenuPair.Key->Title,
				CategoryMenuPair.Key->Tooltip,
				FNewToolMenuDelegate::CreateSP(this, &SAssetFilterBar<FilterType>::CreateFiltersMenuCategory, CategoryMenuPair.Value.Classes),
				FUIAction(
				FExecuteAction::CreateSP(this, &SAssetFilterBar<FilterType>::FilterByTypeCategory, CategoryMenuPair.Key, CategoryMenuPair.Value.Classes),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(this, &SAssetFilterBar<FilterType>::IsTypeCategoryChecked, CategoryMenuPair.Key, CategoryMenuPair.Value.Classes)),
				EUserInterfaceActionType::ToggleButton
				);
		}

		// Now add all non-asset filters
		this->PopulateCustomFilters(Menu);
	}

	/* Asset Type filter related functionality */

	/** Handler to determine the "checked" state of class filter in the filter dropdown */
	bool IsClassTypeInUse(TSharedPtr<FCustomClassFilterData> Class) const
	{
		for (const TSharedPtr<SAssetFilter>& AssetFilter : this->AssetFilters)
		{
			if (AssetFilter.IsValid() && AssetFilter->GetCustomClassFilterData() == Class)
			{
				return true;
			}
		}

		return false;
	}

 	/** Handler to determine the "checked" state of an type category in the filter dropdown */
 	ECheckBoxState IsTypeCategoryChecked(TSharedPtr<FFilterCategory> TypeCategory, TArray<TSharedPtr<FCustomClassFilterData>> Classes) const
	{
		bool bIsAnyActionInUse = false;
		bool bIsAnyActionNotInUse = false;

		for (const TSharedPtr<FCustomClassFilterData>& CustomClassFilter : Classes)
		{
			if (IsClassTypeInUse(CustomClassFilter))
			{
				bIsAnyActionInUse = true;
			}
			else
			{
				bIsAnyActionNotInUse = true;
			}

			if (bIsAnyActionInUse && bIsAnyActionNotInUse)
			{
				return ECheckBoxState::Undetermined;
			}
		}

		if (bIsAnyActionInUse)
		{
			return ECheckBoxState::Checked;
		}
		else
		{
			return ECheckBoxState::Unchecked;
		}
	}

 	/** Function to check if a given type category is in use */
 	bool IsTypeCategoryInUse(TSharedPtr<FFilterCategory> TypeCategory, TArray<TSharedPtr<FCustomClassFilterData>> Classes) const
 	{
 		ECheckBoxState AssetTypeCategoryCheckState = IsTypeCategoryChecked(TypeCategory, Classes);

 		if (AssetTypeCategoryCheckState == ECheckBoxState::Unchecked)
 		{
 			return false;
 		}

 		// An asset type category is in use if any of its type actions are in use (ECheckBoxState::Checked or ECheckBoxState::Undetermined)
 		return true;
 	}

	void OnCopyFilters() const
	{
		if (const FFilterBarSettings* SettingsToCopyFrom = GetConstConfig())
		{
			FString Output = TEXT("");
			FFilterBarSettings::StaticStruct()->ExportText(Output, SettingsToCopyFrom, nullptr, nullptr, (PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited | PPF_IncludeTransient), nullptr);
			FPlatformApplicationMisc::ClipboardCopy(*Output);
		}
	}

	void OnPasteFilters()
	{
		if (FFilterBarSettings* SettingsToPasteTo = GetMutableConfig())
		{
			FString Output = TEXT("");
			FPlatformApplicationMisc::ClipboardPaste(Output);

			FFilterBarSettings::StaticStruct()->ImportText(*Output, SettingsToPasteTo, nullptr, 0, GLog, GetPathNameSafe(FFilterBarSettings::StaticStruct()));

			LoadSettings();
		}
	}

	bool CanPasteFilters() const
	{
		FString Output = TEXT("");
		FPlatformApplicationMisc::ClipboardPaste(Output);

		FCustomizableTextObjectFactory Factory(GWarn);
		if (!Factory.CanCreateObjectsFromText(Output))
		{
			return false;
		}

		FFilterBarSettings Value;
		return FFilterBarSettings::StaticStruct()->ImportText(*Output, &Value, nullptr, 0, GLog, GetPathNameSafe(FFilterBarSettings::StaticStruct())) != nullptr;
	}

protected:
	/** The command list we bind to */
	TWeakPtr<FUICommandList> CommandListWeak;

	/** A copy of all AssetFilters in this->Filters for convenient access */
	TArray< TSharedRef<SAssetFilter> > AssetFilters;

	/** List of custom Class Filters that will be shown in the filter bar */
	TArray<TSharedRef<FCustomClassFilterData>> CustomClassFilters;

	/** Copy of Custom Class Filters that were provided by the user, and not autopopulated from AssetTypeActions */
	TArray<TSharedRef<FCustomClassFilterData>> UserCustomClassFilters;

	FName FilterMenuName;

	TMap<FName, TSharedPtr<FFilterCategory>> AssetFilterCategories;
	
	/** Whether the filter bar provides the default Asset Filters */
	bool bUseDefaultAssetFilters = false;

	/** The AssetClassAction used to get the permission list from the Asset Tools Module */
	EAssetClassAction AssetClassAction = EAssetClassAction::ViewAsset;

	/** A map of any type filters we encounter while calling LoadSettings(), but don't have in this instance of the filter
	 *  bar. These are stored so we can save these when calling SaveSettings() so that they are not lost */
	TMap<FString, bool> UnknownTypeFilters;

private:
	/** The filter menu category to expand. */
	TOptional<FAssetCategoryPath> DefaultMenuExpansionCategory;

	FDelegateHandle AssetRegistryVersionChangeHandle;
};

template <typename FilterType>
SAssetFilterBar<FilterType>::~SAssetFilterBar()
{
	if (UAssetDefinitionRegistry* AssetDefinitionRegistry = UAssetDefinitionRegistry::Get())
	{
		AssetDefinitionRegistry->OnAssetDefinitionRegistryVersionChange().Remove(AssetRegistryVersionChangeHandle);
	}
}

#undef LOCTEXT_NAMESPACE

#undef UE_API
