// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlateGlobals.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"

#include "Filters/SAssetFilterBar.h"
#include "Filters/AssetFilter.h"

/**
* A Filter Bar widget, which can be used to filter items of type [FilterType] given a list of custom filters
* along with built in support for Asset Type filters
* @see SBasicFilterBar if you don't want Asset Type filters, or if you want a filter bar usable in non-editor situations
* NOTE: You will need to also add "ToolWidgets" as a dependency to your module to use this widget
* NOTE: The filter functions create copies, so you want to use SFilterBar<TSharedPtr<ItemType>> etc instead of SFilterBar<ItemType> when possible
* NOTE: The user must specify one of the following:
* a) OnConvertItemToAssetData:   A conversion function to convert FilterType to an FAssetData. Specifying this will filter the asset type
*							     through the AssetRegistry, which is potentially more thorough and fast
* b) OnCompareItemWithClassName: A comparison function to check if an Item (FilterType) is the same as an AssetType (represented by an FName).
*								 This allows to user to directly text compare with Class Names (UObjectBase::GetFName), which is easier but potentially
*								 slower
*								 
* Sample Usage:
*		SAssignNew(MyFilterBar, SFilterBar<FText>)
*		.OnFilterChanged() // A delegate for when the list of filters changes
*		.CustomFilters() // An array of filters available to this FilterBar (@see FGenericFilter to create simple delegate based filters)
*		.OnConvertItemToAssetData() // Conversion Function as mentioned above
*
* Use the GetAllActiveFilters() function to get the FilterCollection of Active Filters in this FilterBar, that can be used to filter your items.
* NOTE: GetAllActiveFilters() must be called every time the filters change (in OnFilterChanged() for example) to make sure you have the correct backend filter
* NOTE: Use CustomClassFilters to provide any Type Filters to make sure they get resolved properly (See FCustomClassFilterData)
* NOTE: Use MakeAddFilterButton() to make the button that summons the dropdown showing all the filters
*/
template<typename FilterType>
class SFilterBar : public SAssetFilterBar<FilterType>
{
public:
	using FOnFilterChanged = typename SBasicFilterBar<FilterType>::FOnFilterChanged;
	using FConvertItemToAssetData = typename FAssetFilter<FilterType>::FConvertItemToAssetData;
	using FCompareItemWithClassNames = typename FAssetFilter<FilterType>::FCompareItemWithClassNames;
	using FCreateTextFilter = typename SBasicFilterBar<FilterType>::FCreateTextFilter;
	using SFilter = typename SBasicFilterBar<FilterType>::SFilter;
	
	SLATE_BEGIN_ARGS( SFilterBar<FilterType> )
		: _FilterBarIdentifier(NAME_None)
		, _UseDefaultAssetFilters(true)
	    , _FilterBarLayout(EFilterBarLayout::Horizontal)
		, _CanChangeOrientation(false)
		, _FilterPillStyle(EFilterPillStyle::Default)
		, _DefaultMenuExpansionCategory(EAssetCategoryPaths::Basic)
		{
		
		}

		/** Delegate for when filters have changed */
		SLATE_EVENT( FOnFilterChanged, OnFilterChanged )

		/** Specify this delegate to use asset comparison through IAssetRegistry, where you specify how to convert your Item into an FAssetData */
		SLATE_EVENT( FConvertItemToAssetData, OnConvertItemToAssetData )

		/** Specify this delegate to use simple asset comparison, where you compare your Item a list of FNames representing Classes */
		SLATE_EVENT( FCompareItemWithClassNames, OnCompareItemWithClassNames )

		/** Initial List of Custom Filters that will be added to the AddFilter Menu */
		SLATE_ARGUMENT( TArray<TSharedRef<FFilterBase<FilterType>>>, CustomFilters)

		/** A unique identifier for this filter bar needed to enable saving settings in a config file */
		SLATE_ARGUMENT(FName, FilterBarIdentifier)
	
		/** Delegate to extend the Add Filter dropdown */
		SLATE_EVENT( FOnExtendAddFilterMenu, OnExtendAddFilterMenu )

		/** Initial List of Custom Class Filters that will be added to the AddFilter Menu */
		SLATE_ARGUMENT( TArray<TSharedRef<FCustomClassFilterData>>, CustomClassFilters)
	
		/** Whether the filter bar should provide the default asset filters */
		SLATE_ARGUMENT(bool, UseDefaultAssetFilters)

		/** A delegate to create a TTextFilter for FilterType items. If provided, will allow creation of custom text filters
		 *  from the filter dropdown menu.
		 */
		SLATE_ARGUMENT(FCreateTextFilter, CreateTextFilter)
			
		/** An SFilterSearchBox that can be attached to this filter bar. When provided along with a CreateTextFilter
		 *  delegate, allows the user to save searches from the Search Box as text filters for the filter bar.
		 *	NOTE: Will bind a delegate to SFilterSearchBox::OnClickedAddSearchHistoryButton
		 */
		SLATE_ARGUMENT(TSharedPtr<SFilterSearchBox>, FilterSearchBox)

		/** The layout that determines how the filters are laid out */
		SLATE_ARGUMENT(EFilterBarLayout, FilterBarLayout)
			
		/** If true, allow dynamically changing the orientation and saving in the config */
		SLATE_ARGUMENT(bool, CanChangeOrientation)
		
		/** Determines how each individual filter pill looks like */
		SLATE_ARGUMENT(EFilterPillStyle, FilterPillStyle)

		/** The filter menu category to expand. */
		SLATE_ARGUMENT(TOptional<FAssetCategoryPath>, DefaultMenuExpansionCategory);
	
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs )
	{
		typename SAssetFilterBar<FilterType>::FArguments Args;
		Args._OnFilterChanged = InArgs._OnFilterChanged;
		Args._CustomFilters = InArgs._CustomFilters;
		Args._UseDefaultAssetFilters = InArgs._UseDefaultAssetFilters;
		Args._CustomClassFilters = InArgs._CustomClassFilters;
		Args._OnExtendAddFilterMenu = InArgs._OnExtendAddFilterMenu;
		Args._CreateTextFilter = InArgs._CreateTextFilter;
		Args._FilterSearchBox = InArgs._FilterSearchBox;
		Args._FilterBarIdentifier = InArgs._FilterBarIdentifier;
		Args._FilterBarLayout = InArgs._FilterBarLayout;
        Args._CanChangeOrientation = InArgs._CanChangeOrientation;
		Args._FilterPillStyle = InArgs._FilterPillStyle;
		Args._DefaultMenuExpansionCategory = InArgs._DefaultMenuExpansionCategory;
		
		SAssetFilterBar<FilterType>::Construct(Args);
		
		// Create the dummy filter that represents all currently active Asset Type filters
		AssetFilter = MakeShareable(new FAssetFilter<FilterType>);

		// Asset Conversion is preferred to Asset Comparison
		if(InArgs._OnConvertItemToAssetData.IsBound())
		{
			AssetFilter->SetConversionFunction(InArgs._OnConvertItemToAssetData);
		}
		else if(InArgs._OnCompareItemWithClassNames.IsBound())
		{
			AssetFilter->SetComparisonFunction(InArgs._OnCompareItemWithClassNames);
		}
		else
		{
			UE_LOGF(LogSlate, Error, "SFilterBar Requires that you specify either OnConvertItemToAssetData or OnCompareItemWithClassName");
		}

		this->ActiveFilters->Add(AssetFilter);
	}
	
	/** Use this function to get all currently active filters (to filter your items)
	 * NOTE: Must be called every time the filters change (OnFilterChanged) to make sure you get the correct combined filter
	 */
	virtual TSharedPtr< TFilterCollection<FilterType> > GetAllActiveFilters() override
	{
		UpdateAssetFilter();
		
		return this->ActiveFilters;	
	}

private:

	void UpdateAssetFilter()
	{
		/** We have to make sure to update the CombinedBackendFilter everytime the user requests all the Filters */
		FARFilter CombinedBackendFilter = this->GetCombinedBackendFilter();
		AssetFilter->SetBackendFilter(CombinedBackendFilter);
	}
	
private:
	/* The invisible filter used to conduct Asset Type filtering */
	TSharedPtr<FAssetFilter<FilterType>> AssetFilter;
};
