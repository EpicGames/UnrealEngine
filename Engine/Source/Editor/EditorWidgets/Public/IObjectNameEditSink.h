// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

namespace UE::EditorWidgets
{

/** ObjectNameEditSink provides an interface for the in-editor object name */
class IObjectNameEditSink
{
public:
	/** Virtual destructor */
	virtual ~IObjectNameEditSink() = default;

	/** Checks to see if the specified object is handled by this type. */
	virtual UClass* GetSupportedClass() const = 0;

	/** Returns the display name for that object. */
	virtual FText GetObjectDisplayName(UObject* Object) const = 0;

	/** Whether or not the name can be edited */
	virtual bool IsObjectDisplayNameReadOnly(UObject* Object) const
	{
		return true;
	};

	/** 
	 * Attempt to rename the object. Will fail if IsObjectDisplayNameReadOnly returns true.
	 * @return Whether or not the display name was successfully set
	 */
	virtual bool SetObjectDisplayName(UObject* Object, FString DisplayName)
	{
		return false;
	};

	/** Returns the tooltip text for that object */
	virtual FText GetObjectNameTooltip(UObject* Object) const = 0;

	/** Whether or not the name has a suffix */
	virtual bool ObjectDisplayNameHasSuffix(const UObject* Object) const
	{
		return false;
	};

	/** Returns the displayed suffix for that object. */
	virtual FText GetObjectDisplayNameSuffix(const UObject* Object) const
	{
		return FText();
	}

};

} //end namespace UE::EditorWidgets
