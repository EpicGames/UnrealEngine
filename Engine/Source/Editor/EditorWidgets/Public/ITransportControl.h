// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Misc/EnumRange.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"

namespace EPlaybackMode
{
	enum Type
	{
		Stopped,
		PlayingForward,
		PlayingReverse,
	};
}

DECLARE_DELEGATE_RetVal(bool, FOnGetLooping)
DECLARE_DELEGATE_RetVal(bool, FOnGetRecording)
DECLARE_DELEGATE_RetVal(EPlaybackMode::Type, FOnGetPlaybackMode)
DECLARE_DELEGATE_TwoParams(FOnTickPlayback, double /*InCurrentTime*/, float /*InDeltaTime*/)
DECLARE_DELEGATE_RetVal(TSharedRef<SWidget>, FOnMakeTransportWidget)
DECLARE_DELEGATE_RetVal(EVisibility, FOnGetRecordButtonVisibility)

enum class ETransportControlWidgetType : int32
{
	BackwardEnd,
	BackwardStep,
	BackwardPlay,
	Record,
	ForwardPlay,
	ForwardStep,
	ForwardEnd,
	Loop,
	Custom
};

ENUM_RANGE_BY_COUNT(ETransportControlWidgetType, ETransportControlWidgetType::Custom);

/** Descriptor for a transport control widget */
struct FTransportControlWidget
{
	FTransportControlWidget(ETransportControlWidgetType InWidgetType)
		: WidgetType(InWidgetType)
	{}

	FTransportControlWidget(FOnMakeTransportWidget InMakeCustomWidgetDelegate)
		: WidgetType(ETransportControlWidgetType::Custom)
		, MakeCustomWidgetDelegate(InMakeCustomWidgetDelegate)
	{}

	/** Basic widget type */
	ETransportControlWidgetType WidgetType;

	/** Delegate used for making custom widgets */
	FOnMakeTransportWidget MakeCustomWidgetDelegate;
};

struct FTransportControlArgs
{
	FTransportControlArgs()
		: OnForwardPlay()
		, OnRecord()
		, OnBackwardPlay()
		, OnForwardStep()
		, OnBackwardStep()
		, OnForwardEnd()
		, OnBackwardEnd()
		, OnToggleLooping()
		, OnGetLooping()
		, OnGetPlaybackMode()
		, OnTickPlayback()
		, OnGetRecording()
		, bAreButtonsFocusable(true)
		, OnGetRecordButtonVisibility()
	{}

	FOnClicked OnForwardPlay;
	FOnClicked OnRecord;
	FOnClicked OnBackwardPlay;
	FOnClicked OnForwardStep;
	FOnClicked OnBackwardStep;
	FOnClicked OnForwardEnd;
	FOnClicked OnBackwardEnd;
	FOnClicked OnToggleLooping;
	FOnGetLooping OnGetLooping;
	FOnGetPlaybackMode OnGetPlaybackMode;
	FOnTickPlayback OnTickPlayback;
	FOnGetRecording OnGetRecording;
	bool bAreButtonsFocusable;
	FOnGetRecordButtonVisibility OnGetRecordButtonVisibility;

	/** 
	 * Array of custom widgets to create - if this array is used the default widget ordering will be
	 * ignored in favor of this set of widgets
	 */
	TArray<FTransportControlWidget> WidgetsToCreate;
};

/**
 * Base class for a widget that allows transport control
 */
class ITransportControl : public SCompoundWidget
{
public:
};
