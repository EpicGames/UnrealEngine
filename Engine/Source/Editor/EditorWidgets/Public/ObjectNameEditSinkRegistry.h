// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IObjectNameEditSink.h"
#include "Templates/SharedPointer.h"

#define UE_API EDITORWIDGETS_API

class UClass;

namespace UE::EditorWidgets
{
class IObjectNameEditSink;

class FObjectNameEditSinkRegistry
{
public:
	UE_API FObjectNameEditSinkRegistry();

	/** Registers an object name edit sink so it can provide a name editing interface for an object type. */
	UE_API void RegisterObjectNameEditSink(const TSharedRef<IObjectNameEditSink>& NewSink);

	/** Unregisters a name edit sink. It will no longer provide a name editing interface for an object type. */
	UE_API void UnregisterObjectNameEditSink(const TSharedRef<IObjectNameEditSink>& SinkToRemove);

	/** Gets the appropriate ObjectNameEditSink for the supplied class */
	UE_API TSharedPtr<IObjectNameEditSink> GetObjectNameEditSinkForClass(const UClass* Class) const;

private:
	/** The list of all registered ObjectNameEditSinks */
	TArray<TSharedRef<IObjectNameEditSink>> ObjectNameEditSinkList;
};

} // end namespace UE::EditorWidgets

#undef UE_API
