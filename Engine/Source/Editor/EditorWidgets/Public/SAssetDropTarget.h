// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "SDropTarget.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API EDITORWIDGETS_API

class FDragDropEvent;
class FDragDropOperation;
class SWidget;
class UObject;
struct FGeometry;

/**
 * A widget that displays a hover cue and handles dropping assets of allowed types onto this widget
 */
class SAssetDropTarget : public SDropTarget
{
public:
	/** Called when a valid asset is dropped */
	DECLARE_DELEGATE_OneParam( FOnAssetDropped, UObject* );

	/** Called when valid assets are dropped */
	DECLARE_DELEGATE_TwoParams( FOnAssetsDropped, const FDragDropEvent&, TArrayView<FAssetData> );

	/** Called when we need to check if an asset type is valid for dropping */
	DECLARE_DELEGATE_RetVal_OneParam( bool, FIsAssetAcceptableForDrop, const UObject* );

	/** Called when we need to check if an asset type is valid for dropping */
	DECLARE_DELEGATE_RetVal_OneParam( bool, FAreAssetsAcceptableForDrop, TArrayView<FAssetData> );

	/** Called when we need to check if an asset type is valid for dropping and also will have a reason if it is not */
	DECLARE_DELEGATE_RetVal_TwoParams( bool, FIsAssetAcceptableForDropWithReason, const UObject*, FText& );

	/** Called when we need to check if an asset type is valid for dropping and also will have a reason if it is not */
	DECLARE_DELEGATE_RetVal_TwoParams( bool, FAreAssetsAcceptableForDropWithReason, TArrayView<FAssetData>, FText& );

	SLATE_BEGIN_ARGS(SAssetDropTarget)
		: _bSupportsMultiDrop(false)
	{ }
		/* Content to display for the in the drop target */
		SLATE_DEFAULT_SLOT( FArguments, Content )
		/** Called when a valid asset is dropped */
		SLATE_EVENT( FOnAssetsDropped, OnAssetsDropped )
		/** Called to check if an asset is acceptable for dropping */
		SLATE_EVENT( FAreAssetsAcceptableForDrop, OnAreAssetsAcceptableForDrop )
		/** Called to check if an asset is acceptable for dropping if you also plan on returning a reason text */
		SLATE_EVENT( FAreAssetsAcceptableForDropWithReason, OnAreAssetsAcceptableForDropWithReason )
		/** Sets if this drop target can support multiple assets dropped, or only supports a single asset dropped at a time. False by default for legacy behavior. */
		SLATE_ARGUMENT( bool, bSupportsMultiDrop )
		/** When this is true, the drop target will only get recognized when entering while drag & dropping. */
		SLATE_ATTRIBUTE(bool, bOnlyRecognizeOnDragEnter)

		FOnAssetsDropped ConvertObjectDropDelegate(const FOnAssetDropped& LegacyDelegate)
		{
			return FOnAssetsDropped::CreateLambda([LegacyDelegate](const FDragDropEvent&, TArrayView<FAssetData> InAssets)
			{
				if (InAssets.Num() == 1)
				{
					LegacyDelegate.Execute(InAssets[0].GetAsset());
				}
			});
		}
		SLATE_EVENT_DEPRECATED(5.0, "Use FOnAssetsDropped instead.", FOnAssetDropped, OnAssetDropped, OnAssetsDropped, ConvertObjectDropDelegate)

		FAreAssetsAcceptableForDrop ConvertObjectAcceptableDropDelegate(const FIsAssetAcceptableForDrop& LegacyDelegate)
		{
			return FAreAssetsAcceptableForDrop::CreateLambda([LegacyDelegate](TArrayView<FAssetData> InAssets)
			{
				if (InAssets.Num() == 1)
				{
					 return LegacyDelegate.Execute(InAssets[0].GetAsset());
				}

				return false;
			});
		}
		SLATE_EVENT_DEPRECATED(5.0, "Use FAreAssetsAcceptableForDrop instead.", FIsAssetAcceptableForDrop, OnIsAssetAcceptableForDrop, OnAreAssetsAcceptableForDrop, ConvertObjectAcceptableDropDelegate)

		FAreAssetsAcceptableForDropWithReason ConvertObjectAcceptableDropWithReasonDelegate(const FIsAssetAcceptableForDropWithReason& LegacyDelegate)
		{
			return FAreAssetsAcceptableForDropWithReason::CreateLambda([LegacyDelegate](TArrayView<FAssetData> InAssets, FText& OutText)
			{
				if (InAssets.Num() == 1)
				{
					 return LegacyDelegate.Execute(InAssets[0].GetAsset(), OutText);
				}

				OutText = NSLOCTEXT("SAssetDropTarget", "NoDroppedArgs", "No valid objects were dropped.");
				return false;
			});
		}
		SLATE_EVENT_DEPRECATED(5.0, "Use FAreAssetsAcceptableForDropWithReason instead.", FIsAssetAcceptableForDropWithReason, OnIsAssetAcceptableForDropWithReason, OnAreAssetsAcceptableForDropWithReason, ConvertObjectAcceptableDropWithReasonDelegate)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs );

protected:
	UE_API FReply OnDropped(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);
	UE_API virtual bool OnAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation) const override;
	UE_API virtual bool OnIsRecognized(TSharedPtr<FDragDropOperation> DragDropOperation) const override;
	UE_API virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;

private:
	UE_API TArray<FAssetData> GetDroppedAssets(TSharedPtr<FDragDropOperation> DragDropOperation, bool& bOutRecognizedEvent) const;

private:
	/** Delegate to call when an asset is dropped */
	FOnAssetsDropped OnAssetsDropped;
	/** Delegate to call to check validity of the asset */
	FAreAssetsAcceptableForDrop OnAreAssetsAcceptableForDrop;
	/** Delegate to call to check validity of the asset if you will also provide a reason when returning false */
	FAreAssetsAcceptableForDropWithReason OnAreAssetsAcceptableForDropWithReason;
	/** Boolean to determine if this drop target can support multiple assets dropped, or only supports a single asset dropped at a time. False by default for legacy behavior. */
	bool bSupportsMultiDrop;
};

#undef UE_API
