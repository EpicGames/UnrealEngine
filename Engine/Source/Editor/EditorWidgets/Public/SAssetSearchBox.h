// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Filters/SFilterSearchBox.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SlateFwd.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define UE_API EDITORWIDGETS_API

class ITableRow;
class SMenuAnchor;
class STableViewBase;
struct FFocusEvent;
struct FGeometry;
struct FKeyEvent;

/** External suggestion entry provided to SAssetSearchBox */
struct FAssetSearchBoxSuggestion
{
	/** The raw suggestion string that should be used with the search box */
	FString SuggestionString;

	/** The user-facing display name of this suggestion */
	FText DisplayName;

	/** The user-facing category name of this suggestion (if any) */
	FText CategoryName;

	static FAssetSearchBoxSuggestion MakeSimpleSuggestion(FString InSuggestionString)
	{
		FAssetSearchBoxSuggestion SimpleSuggestion;
		SimpleSuggestion.SuggestionString = MoveTemp(InSuggestionString);
		SimpleSuggestion.DisplayName = FText::FromString(SimpleSuggestion.SuggestionString);
		return SimpleSuggestion;
	}
};

/** A delegate for a callback to filter the given suggestion list, to allow custom filtering behavior */
DECLARE_DELEGATE_ThreeParams(FOnAssetSearchBoxSuggestionFilter, const FText& /*SearchText*/, TArray<FAssetSearchBoxSuggestion>& /*PossibleSuggestions*/, FText& /*SuggestionHighlightText*/);

/** A delegate for a callback when a suggestion entry is chosen during an asset search, to allow custom compositing behavior of the suggestion into the search text */
DECLARE_DELEGATE_RetVal_TwoParams(FText, FOnAssetSearchBoxSuggestionChosen, const FText& /*SearchText*/, const FString& /*Suggestion*/);

/**
 * A widget to provide a search box with a filtered dropdown menu.
 * Also provides the functionality to show search history optionally, and a button/delegate to save a search via
 * SetOnSaveSearchHandler
 */

class SAssetSearchBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAssetSearchBox)
		: _SuggestionListPlacement( MenuPlacement_BelowAnchor )
		, _OnTextChanged()
		, _OnTextCommitted()
		, _InitialText()
		, _HintText()
		, _PossibleSuggestions(TArray<FAssetSearchBoxSuggestion>())
		, _DelayChangeNotificationsWhileTyping( false )
		, _MustMatchPossibleSuggestions( false )
		, _ShowSearchHistory(false)
		, _SuggestionListHeight(250.f)
	{}

		/** Where to place the suggestion list */
		SLATE_ARGUMENT( EMenuPlacement, SuggestionListPlacement )

		/** Invoked whenever the text changes */
		SLATE_EVENT( FOnTextChanged, OnTextChanged )

		/** Invoked whenever the text is committed (e.g. user presses enter) */
		SLATE_EVENT( FOnTextCommitted, OnTextCommitted )

		/** Initial text to display for the search text */
		SLATE_ATTRIBUTE( FText, InitialText )

		/** Hint text to display for the search text when there is no value */
		SLATE_ATTRIBUTE( FText, HintText )

		/** All possible suggestions for the search text */
		SLATE_ATTRIBUTE( TArray<FAssetSearchBoxSuggestion>, PossibleSuggestions )

		/** Whether the SearchBox should delay notifying listeners of text changed events until the user is done typing */
		SLATE_ATTRIBUTE( bool, DelayChangeNotificationsWhileTyping )

		/** Whether the SearchBox allows entries that don't match the possible suggestions */
		SLATE_ATTRIBUTE( bool, MustMatchPossibleSuggestions )

		/** Callback to filter the given suggestion list, to allow custom filtering behavior */
		SLATE_EVENT( FOnAssetSearchBoxSuggestionFilter, OnAssetSearchBoxSuggestionFilter )

		/** Callback when a suggestion entry is chosen during an asset search, to allow custom compositing behavior of the suggestion into the search text */
		SLATE_EVENT( FOnAssetSearchBoxSuggestionChosen, OnAssetSearchBoxSuggestionChosen )

		/** Callback delegate to have first chance handling of the OnKeyDown event */
		SLATE_EVENT( FOnKeyDown, OnKeyDownHandler )
	
		/** Whether we should show a dropdown containing the last few searches */
		SLATE_ATTRIBUTE(bool, ShowSearchHistory)

		/** Handler for when the + Button next to a search is clicked */
		SLATE_EVENT(SFilterSearchBox::FOnSaveSearchClicked, OnSaveSearchClicked)

		/** Height of the suggestion list dropdown */
		SLATE_ARGUMENT(float, SuggestionListHeight)

		/**
		 * Optional widget to display on the right side of the search box, after the existing buttons.
		 * EXPERIMENTAL: This API is subject to change.
		 */
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		SLATE_ARGUMENT(TSharedPtr<SWidget>, ExtraWidget)
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct( const FArguments& InArgs );

	/** Sets the text string currently being edited */
	UE_API void SetText(const TAttribute< FText >& InNewText);
	
	/** Get the text string currently being edited */
	UE_API FText GetText() const;

	/** Set or clear the current error reporting information for this search box */
	UE_API void SetError( const FText& InError );
	UE_API void SetError( const FString& InError );

	/** Show a + button next to the current search and set the handler for when that is clicked */
	UE_API void SetOnSaveSearchHandler(SFilterSearchBox::FOnSaveSearchClicked InOnSaveSearchHandler);
	
	// SWidget implementation
	UE_API virtual FReply OnPreviewKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	UE_API virtual bool SupportsKeyboardFocus() const override;
	UE_API virtual bool HasKeyboardFocus() const override;
	UE_API virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override;

private:
	struct FSuggestionListEntry
	{
		FString Suggestion;
		FText DisplayName;
		bool bIsHeader = false;
	};

	/** First chance handler for key down events to the editable text widget */
	UE_API FReply HandleKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/** Handler for when text in the editable text box changed */
	UE_API void HandleTextChanged(const FText& NewText);

	/** Handler for when text in the editable text box changed */
	UE_API void HandleTextCommitted(const FText& NewText, ETextCommit::Type CommitType);

	/** Called by SListView when the selection changes in the suggestion list */
	UE_API void OnSelectionChanged( TSharedPtr<FSuggestionListEntry> NewValue, ESelectInfo::Type SelectInfo );

	/** Makes the widget for a suggestion message in the list view */
	UE_API TSharedRef<ITableRow> MakeSuggestionListItemWidget(TSharedPtr<FSuggestionListEntry> Suggestion, const TSharedRef<STableViewBase>& OwnerTable);

	/** Gets the text to highlight in the suggestion list */
	UE_API FText GetHighlightText() const;

	/** Updates and shows or hides the suggestion list */
	UE_API void UpdateSuggestionList();

	/** Sets the focus to the InputText box */
	UE_API void FocusEditBox();

	/** Returns the currently selected suggestion */
	UE_API TSharedPtr<FSuggestionListEntry> GetSelectedSuggestion() const;

	/** Default implementation of OnAssetSearchBoxSuggestionFilter, if no external implementation is provided */
	static UE_API void DefaultSuggestionFilterImpl(const FText& SearchText, TArray<FAssetSearchBoxSuggestion>& PossibleSuggestions, FText& SuggestionHighlightText);

	/** Default implementation of OnAssetSearchBoxSuggestionChosen, if no external implementation is provided */
	static UE_API FText DefaultSuggestionChosenImpl(const FText& SearchText, const FString& Suggestion);

	/** Returns the width for the suggestion list */
	UE_API FOptionalSize GetSuggestionListWidth() const;

	/** The editable text field */
	TSharedPtr< SFilterSearchBox > InputText;

	/** The the state of the text prior to being committed */
	FText PreCommittedText;

	/** The highlight text to use for the suggestions list */
	FText SuggestionHighlightText;

	/** Auto completion elements */
	TSharedPtr< SMenuAnchor > SuggestionBox;

	/** All suggestions stored in this widget for the list view */
	TArray< TSharedPtr<FSuggestionListEntry> > Suggestions;

	/** The list view for showing all suggestions */
	TSharedPtr< SListView< TSharedPtr<FSuggestionListEntry> > > SuggestionListView;

	/** Delegate to filter the given suggestion list, to allow custom filtering behavior */
	FOnAssetSearchBoxSuggestionFilter OnAssetSearchBoxSuggestionFilter;

	/** Delegate when a suggestion entry is chosen during an asset search, to allow custom compositing behavior of the suggestion into the search text */
	FOnAssetSearchBoxSuggestionChosen OnAssetSearchBoxSuggestionChosen;

	/** Delegate for when text is changed in the edit box */
	FOnTextChanged OnTextChanged;

	/** Delegate for when text is changed in the edit box */
	FOnTextCommitted OnTextCommitted;

	/** Delegate for first chance handling for key down events */
	FOnKeyDown OnKeyDownHandler;

	/** All possible suggestions for the search text */
	TAttribute< TArray<FAssetSearchBoxSuggestion> > PossibleSuggestions;

	/** Determines whether or not the committed text should match a suggestion */
	bool bMustMatchPossibleSuggestions;
};

#undef UE_API
