// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Framework/SlateDelegates.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/StyleColors.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API EDITORWIDGETS_API

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
class SWidget;
struct FGeometry;
struct FSlateBrush;

/** 
 * A widget that displays a hover cue and handles dropping assets of allowed types onto this widget
 */
class SDropTarget : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(bool, FVerifyDrag, TSharedPtr<FDragDropOperation>);
	DECLARE_DELEGATE_OneParam(FOnDragAction, const FDragDropEvent&);
	DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnDropDeprecated, TSharedPtr<FDragDropOperation>);

	SLATE_BEGIN_ARGS(SDropTarget)
		: _ValidColor(FStyleColors::AccentBlue)
		, _InvalidColor(FStyleColors::Error)
		, _VerticalImage(FAppStyle::GetBrush("WideDash.Vertical"))
		, _HorizontalImage(FAppStyle::GetBrush("WideDash.Horizontal"))
		, _BackgroundImage(FAppStyle::GetBrush("DropTarget.Background"))
		, _bOnlyRecognizeOnDragEnter(false)
		, _bUseAllowDropCache(false)
	{ }
	
		UE_DEPRECATED(5.0, "BackgroundColor has been removed. You may alter the background brush to get the same effect.")
		FArguments& BackgroundColor(const FLinearColor& InBackgroundColor)
		{
			return Me();
		}

		UE_DEPRECATED(5.0, "BackgroundColorHover has been removed. You may alter the background brush when hovered to get the same effect.")
		FArguments& BackgroundColorHover(const FLinearColor& InBackgroundColor)
		{
			return Me();
		}
		/* Content to display for the in the drop target */
		SLATE_DEFAULT_SLOT( FArguments, Content )
		/** The color of the vertical/horizontal images when the drop data is valid */
		SLATE_ARGUMENT(FSlateColor, ValidColor)
		/** The color of the vertical/horizontal images when the drop data is not valid */
		SLATE_ARGUMENT(FSlateColor, InvalidColor)
		/** Vertical border image that is used. */
		SLATE_ARGUMENT(const FSlateBrush*, VerticalImage)
		/** Horizontal border image that is used. */
		SLATE_ARGUMENT(const FSlateBrush*, HorizontalImage)
		/** The background image that is applied after the surface. */
		SLATE_ATTRIBUTE(const FSlateBrush*, BackgroundImage)
		/** Called when a valid asset is dropped */
		SLATE_EVENT(FOnDrop, OnDropped)
		/** Called to check if an asset is acceptable for dropping */
		SLATE_EVENT(FVerifyDrag, OnAllowDrop)
		/** Called to check if an asset is acceptable for dropping */
		SLATE_EVENT(FVerifyDrag, OnIsRecognized)
		SLATE_EVENT(FOnDragAction, OnDragEnter)
		SLATE_EVENT(FOnDragAction, OnDragLeave)
		/** When this is true, the drop target will only get recognized when entering while drag & dropping. */
		SLATE_ATTRIBUTE(bool, bOnlyRecognizeOnDragEnter)
		/** Whether to cache off the results of AllowDrop. Useful when then OnAllowDrop callback is expensive since it's called per frame. */
		SLATE_ARGUMENT(bool, bUseAllowDropCache)

		FOnDrop ConvertOnDropFn(const FOnDropDeprecated& LegacyDelegate)
		{
			return FOnDrop::CreateLambda([LegacyDelegate](const FGeometry&, const FDragDropEvent& DragDropEvent)
			{
				if (LegacyDelegate.IsBound())
				{
					return LegacyDelegate.Execute(DragDropEvent.GetOperation());
				}

				return FReply::Unhandled();
			});
		}
		SLATE_EVENT_DEPRECATED(5.0, "Use OnDropped instead.", FOnDropDeprecated, OnDrop, OnDropped, ConvertOnDropFn)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs );

	void ClearAllowDropCache()
	{
		AllowDropCache.Reset();
	}
	
protected:

	UE_API bool AllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation) const;

	UE_API virtual bool OnAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation) const;
	UE_API virtual bool OnIsRecognized(TSharedPtr<FDragDropOperation> DragDropOperation) const;
	
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
protected:
	// SWidget interface
	UE_API virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	// End of SWidget interface

	/** @return Visibility of the overlay text when dragging is occurring. */
	UE_API EVisibility GetDragOverlayVisibility() const;

	/** Get the brightness on the background. */
	UE_API FSlateColor GetBackgroundBrightness() const;

	/** Returns true if this drop target is currently being hovered over by a drag drop event */
	bool IsDragOver() const { return bIsDragOver; }

private:
	/** Delegate to call when an asset is dropped */
	FOnDrop DroppedEvent;
	/** Delegate to call to check validity of the asset */
	FVerifyDrag AllowDropEvent;
	/** Delegate to call to check validity of the asset */
	FVerifyDrag IsRecognizedEvent;
	FOnDragAction OnDragEnterEvent;
	FOnDragAction OnDragLeaveEvent;
	/** Attribute to check if the drop target should only be useable when actually dragging over it. */
	TAttribute<bool> bOnlyRecognizeOnDragEnter;
	bool bUseAllowDropCache = false;
	
	/** The color of the vertical/horizontal images when the drop data is valid */
	FSlateColor ValidColor;
	/** The color of the vertical/horizontal images when the drop data is not valid */
	FSlateColor InvalidColor;
	/** Vertical border image that is used. */
	const FSlateBrush* VerticalImage;
	/** Horizontal border image that is used. */
	const FSlateBrush* HorizontalImage;

	/** Whether or not we are being dragged over by a recognized event*/
	mutable bool bIsDragEventRecognized;
	/** Whether or not we currently allow dropping */
	mutable bool bAllowDrop;
	/** Is the drag operation currently over our airspace? */
	mutable bool bIsDragOver;

	mutable TOptional<bool> AllowDropCache;
	bool bIsDragDropping = false;
	bool bWasDragDroppingLastFrame = false;
};

#undef UE_API
