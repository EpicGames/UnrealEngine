// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"

#define UE_API EDITORWIDGETS_API

class SWidget;
class UEnum;
struct FButtonStyle;

class SEnumComboBox : public SComboButton
{
public:
	DECLARE_DELEGATE_TwoParams(FOnEnumSelectionChanged, int32, ESelectInfo::Type);
	DECLARE_DELEGATE_RetVal_OneParam(FText, FOnGetToolTipForValue, int32 /* Value */)

public:
	SLATE_BEGIN_ARGS(SEnumComboBox)
		: _CurrentValue()
		, _ContentPadding(FMargin(4.0, 2.0))
		, _ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton"))
		, _ButtonStyle(nullptr)
		, _TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		, _bForceBitFlags(false)
	{}

		SLATE_ATTRIBUTE(int32, CurrentValue)
		SLATE_ATTRIBUTE(FMargin, ContentPadding)
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
		SLATE_ATTRIBUTE(FText, OverrideNoFlagsSetText)
		SLATE_EVENT(FOnEnumSelectionChanged, OnEnumSelectionChanged)
		SLATE_EVENT(FOnGetToolTipForValue, OnGetToolTipForValue)
		SLATE_STYLE_ARGUMENT(FComboButtonStyle, ComboButtonStyle)
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)
		SLATE_ARGUMENT(TArray<int32>, EnumValueSubset)
		SLATE_ARGUMENT(bool, bForceBitFlags)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const UEnum* InEnum);

private:
	UE_API FText GetCurrentValueText() const;
	UE_API FText GetCurrentValueTooltip() const;
	UE_API TSharedRef<SWidget> OnGetMenuContent();

private:

	struct FEnumInfo
	{
		FEnumInfo() = default;
		FEnumInfo(const int32 InIndex, const int32 InValue, const FText InDisplayName, const FText InTooltipText)
			: Index(InIndex), Value(InValue), DisplayName(InDisplayName), TooltipText(InTooltipText)
		{}
		
		int32 Index = 0;
		int32 Value = 0;
		FText DisplayName;
		FText TooltipText;
	};
	
	const UEnum* Enum;

	TAttribute<int32> CurrentValue;

	TAttribute<FSlateFontInfo> Font;
	TAttribute<FText> OverrideNoFlagsSetText;

	TArray<FEnumInfo> VisibleEnums;

	bool bUpdatingSelectionInternally;

	bool bIsBitflagsEnum;
	
	FOnGetToolTipForValue OnGetToolTipForValue;

	FOnEnumSelectionChanged OnEnumSelectionChangedDelegate;
};

#undef UE_API
