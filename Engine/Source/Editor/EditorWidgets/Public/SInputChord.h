// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/UICommandInfo.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API EDITORWIDGETS_API

/**
 * Displays a given input chord in a user-friendly way.
 * ie. "[Keyboard Icon] Ctrl + C | Copy"
 */
class SInputChord : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SInputChord)
		: _InputLabelStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>(TEXT("NormalText")))
		, _ActionLabelStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>(TEXT("NormalText")))
	{ }
		/** The Icon should be derived from an FInputChord, usually a keyboard or mouse icon. */
		SLATE_ARGUMENT(const FSlateBrush*, Icon)
		/** Label for the input key combination, ie. "Ctrl + C" */
		SLATE_ARGUMENT(FText, InputLabel)
		/** Optionally specify the style of the input label */
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, InputLabelStyle)
		/** Label for the resulting action, ie. "Copy" */
		SLATE_ARGUMENT(FText, ActionLabel)
		/** Optionally specify the style of the action label */
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, ActionLabelStyle)
		/** Optionally override the delimiter widget between the input label and action label (if set). By default, this is a vertical separator. */
		SLATE_ARGUMENT(TSharedPtr<SWidget>, InputLabelDelimiterOverride)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);

	/** Create an SInputChord widget for the given InputChord and (optional) ActionLabel. The InputLabel (ie. "Ctrl + C") is derived from the InputChord. */
	static UE_API TSharedRef<SInputChord> MakeForInputChord(const FInputChord& InInputChord, const FText& InActionLabel = { });

	/** Create an SInputChord widget for the given CommandInfo. Optionally, provide a KeyBindingIndex for commands with multiple bindings. */
	static UE_API TSharedRef<SInputChord> MakeForCommandInfo(const FUICommandInfo& InCommandInfo, const EMultipleKeyBindingIndex InKeyBindingIndex = EMultipleKeyBindingIndex::Primary);

private:
	static UE_API const FSlateBrush* GetIconForInputChord(const FInputChord& InInputChord);
};

#undef UE_API
