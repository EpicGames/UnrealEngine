// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API EDITORWIDGETS_API

class FName;
class FString;
class ITableRow;
class STableViewBase;
struct FMetaDataLine;

/**
 * The widget to display metadata as a table of tag/value rows
 */
class SMetaDataView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMetaDataView)	{}
	SLATE_END_ARGS()

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InMetaData			The metadata tags/values to display in the table view widget
	 */
	UE_API void Construct(const FArguments& InArgs, const TMap<FName, FString>& InMetadata);

private:
	TArray< TSharedPtr< FMetaDataLine > > MetaDataLines;

	UE_API TSharedRef< ITableRow > OnGenerateRow(const TSharedPtr< FMetaDataLine > Item, const TSharedRef< STableViewBase >& OwnerTable);
};

#undef UE_API
