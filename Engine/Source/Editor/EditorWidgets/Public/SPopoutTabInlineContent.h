// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Styling/AppStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

#define UE_API EDITORWIDGETS_API

class FSpawnTabArgs;
class FTabManager;
class FWorkspaceItem;
class SBox;
class SButton;

/**
 * Inline content that can be popped out to a tab window. Manages the registration of the tab window.
 */
class SPopoutTabInlineContent : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal(bool, FOnIsPoppedOut)
	DECLARE_DELEGATE_OneParam(FOnPopoutStateChanged, const bool /*bInPoppedOut*/);

	/** Constructs a default toggle button that can be used to pop out or pop in the content */
	UE_API static TSharedRef<SButton> CreateToggleButton(const TSharedRef<SPopoutTabInlineContent>& InOwner
		, const FText& InTooltipText = NSLOCTEXT("ToolWidgets", "PopoutTabInlineContentToolTip", "Open this content in its own tab window")
		, const FSlateBrush* const InIconBrush = FAppStyle::GetBrush(TEXT("Profiler.EventGraph.ExpandSelection")));

	SLATE_BEGIN_ARGS(SPopoutTabInlineContent)
		: _ContentHAlign(HAlign_Center)
		, _ContentVAlign(VAlign_Center)
		, _TabManager(nullptr)
		, _TabId(NAME_None)
		, _TabDisplayName(FText::GetEmpty())
		, _TabIcon()
		, _TabRole(ETabRole::NomadTab)
		, _TabGroup(nullptr)
		, _Autosize(false)
		, _AutosizeWhenFloating(false)
		, _CanDragToPopout(false)
		, _ShowDragBorder(true)
		, _HideDockTabStackTab(false)
		, _InitiallyPoppedOut(false)
		, _OnPopoutStateChanged()
		, _OnGetMenuContent()
	{}
		/** Inline content to pop out */
		SLATE_DEFAULT_SLOT(FArguments, Content)
		/** Horizontal alignment of the content */
		SLATE_ARGUMENT(EHorizontalAlignment, ContentHAlign)
		/** Vertical alignment of the content */
		SLATE_ARGUMENT(EVerticalAlignment, ContentVAlign)

		/** Explicit tab manager to use, otherwise uses the global tab manager */
		SLATE_ARGUMENT(TWeakPtr<FTabManager>, TabManager)
		/** Unique identifier for the registered tab window */
		SLATE_ARGUMENT(FName, TabId)
		/** Display text to show on the tab */
		SLATE_ARGUMENT(FText, TabDisplayName)
		/** Icon to use for the registered tab window */
		SLATE_ARGUMENT(FSlateIcon, TabIcon)
		/** Type of tab window to register */
		SLATE_ARGUMENT(ETabRole, TabRole)
		/** Tab group that determines where the menu item exists for the tab window */
		SLATE_ARGUMENT(TSharedPtr<FWorkspaceItem>, TabGroup)

		/** If true, will auto size to the content when undocked, docked, and popped inline */
		SLATE_ARGUMENT(bool, Autosize)
		/** If true, will auto size the tab window to the content when it's popped out and floating undocked */
		SLATE_ARGUMENT(bool, AutosizeWhenFloating)
		/** If true, can click and drag in empty space to pop out content */
		SLATE_ARGUMENT(bool, CanDragToPopout)
		/** If true, shows the handle image that helps let the user know they can popout content */
		SLATE_ARGUMENT(bool, ShowDragBorder)
		/** If true, hides the dock tab stack tab */
		SLATE_ARGUMENT(bool, HideDockTabStackTab)
		/** If true, pops out the content as soon as this widget is constructed */
		SLATE_ARGUMENT(bool, InitiallyPoppedOut)

		/** Event called when the popped out state is changed */
		SLATE_EVENT(FOnPopoutStateChanged, OnPopoutStateChanged)
		/** Event called to get content to be displayed for the right click context menu in empty space */
		SLATE_EVENT(FOnGetContent, OnGetMenuContent)
	SLATE_END_ARGS()

	UE_API virtual ~SPopoutTabInlineContent() override;

	UE_API void Construct(const FArguments& InArgs);

	//~ Begin SWidget
	UE_API virtual FCursorReply OnCursorQuery(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) const override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	UE_API virtual void OnMouseLeave(const FPointerEvent& InPointerEvent) override;
	UE_API virtual FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	//~ End SWidget

	/**
	 * Checks whether the inline content is currently popped out as a tab window.
	 *
	 * @return True if the content is popped out, otherwise false.
	 */
	UE_API bool IsPoppedOut() const;

	/**
	 * Modifies the popout state of the inline content.
	 * Popping out the content detaches it from its current container and moves it to a new tab window,
	 * while turning off the popout state reattaches it to its original container.
	 *
	 * @param bInPopout If true, the content is popped out to a separate tab window.
	 *                  If false, the content is returned to its original container.
	 */
	UE_API void Popout(const bool bInPopout);

	/**
	 * Toggles the popout state of the inline content. Switches the content between its popped-out and embedded states.
	 */
	UE_API void TogglePopout();

private:
	/** Gets the passed in tab manager or the global tab manager none was passed or null */
	TSharedPtr<FTabManager> GetTabManager() const;

	/** Checks if the passed in tab Id is unique and does not conflict with any other existing tabs */
	bool IsUniqueTabId() const;

	void RegisterTab();
	void UnregisterTab();

	TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& InArgs);
	void CloseTab();

	void OnTabActivated(const TSharedRef<SDockTab> InDockTab, const ETabActivationCause InActivationCause);
	void OnTabClosed(const TSharedRef<SDockTab> InDockTab);
	void OnTabRelocated();
	void OnTabForegrounded(const TSharedPtr<SDockTab> InNewTab, const TSharedPtr<SDockTab> InPreviousTab);

	TSharedRef<SWidget> CreateDefaultMenuContent();

	EVisibility GetContentVisibility() const;
	EVisibility GetContentBorderVisibility() const;

	void RegisterAutoSizeTimer(const TSharedPtr<SDockTab>& InDockTab = nullptr);
	void UnregisterAutoSizeTimer();
	EActiveTimerReturnType HandleAutoSizeWindowTimer(const double InCurrentTime
		, const float InDeltaTime, const TWeakPtr<SDockTab> InWeakTab);

	void TryHideStackTab();

	void BroadcastPopoutStateChanged(const bool bInNewPoppedOut);

	void RestoreInlineContent();

	void SubscribeToTabForegrounded();
	void UnsubscribeFromTabForegrounded();

	/** @return True if the tab is popped out and the tab is undocked */
	static bool IsTabFloating(const TSharedRef<SDockTab>& InDockTab);

	/** Cached slate constructor arguments */
	TWeakPtr<SWidget> WeakContentWidget;
	TWeakPtr<FTabManager> WeakTabManager;
	FName TabId;
	FText TabDisplayName;
	FSlateIcon TabIcon;
	ETabRole TabRole = ETabRole::NomadTab;
	TWeakPtr<FWorkspaceItem> WeakTabGroup;
	bool bAutosize = false;
	bool bAutosizeWhenFloating = false;
	bool bCanDragToPopout = false;
	bool bShowDragBorder = true;
	bool bHideDockTabStackTab = false;
	FOnPopoutStateChanged PopoutStateChangedDelegate;
	FOnGetContent GetMenuContentDelegate;

	TWeakPtr<SDockTab> WeakSpawnedDockTab;

	bool bTabRegistered = false;
	bool bPoppedOut = false;
	bool bIsForeground = false;
	bool bWantsToPopout = false;

	bool bLeftMouseDownOverWidget = false;
	bool bRightMouseDownOverWidget = false;

	FDelegateHandle TabForegroundedDelegateHandle;
	FOnActiveTabChanged::FDelegate TabForegroundedDelegate;

	TSharedPtr<FActiveTimerHandle> SetupWindowTimerHandle;

	/** Inline container that owns the content when not popped out */
	TSharedPtr<SBox> InlineContentContainer;
	/** Strong ref while content is moved out of the inline host and into the dock tab */
	TSharedPtr<SWidget> DetachedContentWidget;
};

#undef UE_API
