// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"

class FString;
class FText;
class ITextLayoutMarshaller;

class STemplateStringEditableTextBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STemplateStringEditableTextBox)
		: _Style(nullptr)
		, _AllowMultiLine(false)
		, _IsReadOnly(false)
	{}
		/** The styling of the textbox. */
		SLATE_STYLE_ARGUMENT(FEditableTextBoxStyle, Style)

		/** The initial text that will appear in the widget. */
		SLATE_ATTRIBUTE(FText, Text)

		/** The (optional) resolved text, displayed when the text box is not focused. If not provided, the templated text is shown. */
		SLATE_ATTRIBUTE(FText, ResolvedText)

		/** The list of available arguments to use in this template string. If Empty, any argument name is valid. */
		SLATE_ATTRIBUTE(TConstArrayView<FString>, ValidArguments)

		/** The marshaller used to get/set the raw text to/from the text layout. */
		SLATE_ARGUMENT(TSharedPtr<ITextLayoutMarshaller>, Marshaller)

		/** Whether to allow multi-line text. */
		SLATE_ATTRIBUTE(bool, AllowMultiLine)

		/** Sets whether this text box can actually be modified interactively by the user */
		SLATE_ATTRIBUTE(bool, IsReadOnly)

		/** Allows custom validation. */
		SLATE_EVENT(FOnVerifyTextChanged, OnValidateTokenizedText)

		/** Called whenever the (tokenized) text is changed interactively by the user. */
		SLATE_EVENT(FOnTextChanged, OnTextChanged)
		
		/** Called whenever the (tokenized) text is committed by the user. */
		SLATE_EVENT(FOnTextCommitted, OnTextCommitted)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	EDITORWIDGETS_API void Construct(const FArguments& InArgs);

private:
	/** @return Text contents based on current state */
	FText GetText() const;

	const FSlateBrush* GetBorderImage() const;
	EDITORWIDGETS_API virtual FSlateColor GetForegroundColor() const override;

	void SetStyle(const FEditableTextBoxStyle* InStyle);

	void SetError(const FText& InError) const;

	EDITORWIDGETS_API virtual bool HasKeyboardFocus() const override;
	EDITORWIDGETS_API virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	EDITORWIDGETS_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	bool ValidateTokenizedText(const FText& InTokenizedText);
	bool ValidateTokenBraces(const FText& InTokenizedText, FText& OutError);
	bool ValidateTokenArgs(const FText& InTokenizedText, FText& OutError);

	bool ParseArgs(const FString& InTokenizedTextString, TArray<FString>& OutArgs);

	void OnEditableTextChanged(const FText& InTokenizedText);
	void OnEditableTextCommitted(const FText& InTokenizedText, ETextCommit::Type InCommitType);

private:
	/** Editable text widget. */
	TSharedPtr<SMultiLineEditableText> EditableText;

	/** Style shared between Editable and NonEditable text widgets. */
	const FEditableTextBoxStyle* TextBoxStyle = nullptr;

	TSharedPtr<SHorizontalBox> Box;

	TSharedPtr<SBox> HScrollBarBox;
	TSharedPtr<SScrollBar> HScrollBar;

	TSharedPtr<SBox> VScrollBarBox;
	TSharedPtr<SScrollBar> VScrollBar;

	TSharedPtr<class IErrorReportingWidget> ErrorReporting;
	
	TAttribute<FText> TokenizedText;
	TAttribute<FText> ResolvedText;
	TAttribute<TConstArrayView<FString>> ValidArguments;
	
	/** Callback to verify tokenized text when changed. Will return an error message to denote problems. */
    FOnVerifyTextChanged OnValidateTokenizedText;

	/** Callback when tokenized text is changed. */
	FOnTextChanged OnTokenizedTextChanged;

	/** Callback when tokenized text is committed. */
	FOnTextCommitted OnTokenizedTextCommitted;
};
