// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Misc/TextFilter.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SListView.h"

#define UE_API EDITORWIDGETS_API

class SComboButton;
class SEditableTextBox;
class SMultiLineEditableTextBox;
class SSearchBox;
class SWidget;
class UObject;
class UPackage;
struct FFocusEvent;
struct FGeometry;
struct FSlateBrush;

/** Interface to allow STextPropertyEditableTextBox to be used to edit both properties and Blueprint pins */
class IEditableTextProperty
{
public:
	enum class ETextPropertyEditAction : uint8
	{
		EditedNamespace = (uint8)TextNamespaceUtil::ETextEditAction::Namespace,
		EditedKey = (uint8)TextNamespaceUtil::ETextEditAction::Key,
		EditedSource = (uint8)TextNamespaceUtil::ETextEditAction::SourceString,
	};

	virtual ~IEditableTextProperty() = default;

	/** Are the text properties being edited marked as multi-line? */
	virtual bool IsMultiLineText() const = 0;

	/** Are the text properties being edited marked as password fields? */
	virtual bool IsPassword() const = 0;

	/** Are the text properties being edited read-only? */
	virtual bool IsReadOnly() const = 0;

	/** Is the value associated with the properties the default value? */
	virtual bool IsDefaultValue() const = 0;

	/** Get the tooltip text associated with the property being edited */
	virtual FText GetToolTipText() const = 0;

	/** Get the number of FText instances being edited by this property */
	virtual int32 GetNumTexts() const = 0;

	/** Get the text at the given index (check against GetNumTexts) */
	virtual FText GetText(const int32 InIndex) const = 0;

	/** Set the text at the given index (check against GetNumTexts) */
	virtual void SetText(const int32 InIndex, const FText& InText) = 0;

	/** Check to see if the given text is valid to use */
	virtual bool IsValidText(const FText& InText, FText& OutErrorMsg) const = 0;

#if USE_STABLE_LOCALIZATION_KEYS
	/** Get the stable text ID for the given index (check against GetNumTexts) */
	virtual void GetStableTextId(const int32 InIndex, const ETextPropertyEditAction InEditAction, const FString& InTextSource, const FString& InProposedNamespace, const FString& InProposedKey, FString& OutStableNamespace, FString& OutStableKey) const = 0;
#endif // USE_STABLE_LOCALIZATION_KEYS

protected:
#if USE_STABLE_LOCALIZATION_KEYS
	/** Get the localization ID we should use for the given object, and the given text instance */
	EDITORWIDGETS_API static void StaticStableTextId(UObject* InObject, const ETextPropertyEditAction InEditAction, const FString& InTextSource, const FString& InProposedNamespace, const FString& InProposedKey, FString& OutStableNamespace, FString& OutStableKey);

	/** Get the localization ID we should use for the given package, and the given text instance */
	EDITORWIDGETS_API static void StaticStableTextId(UPackage* InPackage, const ETextPropertyEditAction InEditAction, const FString& InTextSource, const FString& InProposedNamespace, const FString& InProposedKey, FString& OutStableNamespace, FString& OutStableKey);
#endif // USE_STABLE_LOCALIZATION_KEYS
};

/** A widget that can be used for editing the string table referenced for FText instances */
class STextPropertyEditableStringTableReference : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(STextPropertyEditableStringTableReference)
		: _ComboStyle(&FCoreStyle::Get().GetWidgetStyle<FComboBoxStyle>("ComboBox"))
		, _ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
		, _AllowUnlink(false)
		{}
		/** The styling of the combobox */
		SLATE_STYLE_ARGUMENT(FComboBoxStyle, ComboStyle)
		/** The styling of the button */
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)
		/** Font for comboboxes */
		SLATE_ARGUMENT(FSlateFontInfo, Font)
		/** Should we show an "unlink" button? */
		SLATE_ARGUMENT(bool, AllowUnlink)
	SLATE_END_ARGS()

public:
	UE_API void Construct(const FArguments& Arguments, const TSharedRef<IEditableTextProperty>& InEditableTextProperty);

private:
	struct FAvailableStringTable
	{
		FName TableId;
		FText DisplayName;
	};

	UE_API void GetTableIdAndKey(FName& OutTableId, FString& OutKey) const;
	UE_API void SetTableIdAndKey(const FName InTableId, const FString& InKey);

	UE_API void OnOptionsFilterTextChanged(const FText& InNewText);
	UE_API void OnKeysFilterTextChanged(const FText& InNewText);

	UE_API TSharedRef<SWidget> OnGetStringTableComboOptions();
	UE_API TSharedRef<class ITableRow> OnGenerateStringTableComboOption(TSharedPtr<FAvailableStringTable> InItem, const TSharedRef<class STableViewBase>& OwnerTable);
	UE_API TSharedRef<SWidget> OnGetStringTableKeyOptions();
	UE_API TSharedRef<class ITableRow> OnGenerateStringTableKeyOption(TSharedPtr<FString> InItem, const TSharedRef<class STableViewBase>& OwnerTable);

	UE_API void OnStringTableComboChanged(TSharedPtr<FAvailableStringTable> NewSelection, ESelectInfo::Type SelectInfo);
	UE_API void UpdateStringTableComboOptions();
	UE_API FText GetStringTableComboContent() const;
	UE_API FText GetStringTableComboToolTip() const;

	UE_API void OnKeyComboChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	UE_API void UpdateStringTableKeyOptions();
	UE_API FText GetKeyComboContent() const;
	UE_API FText GetKeyComboToolTip() const;

	UE_API bool IsUnlinkEnabled() const;
	UE_API FReply OnUnlinkClicked();

	TSharedPtr<IEditableTextProperty> EditableTextProperty;

	using FOptionTextFilter = TTextFilter< TSharedPtr<FAvailableStringTable> >;
	TSharedPtr<FOptionTextFilter> OptionTextFilter;
	TSharedPtr<SSearchBox> OptionsSearchBox;

	using FKeyTextFilter = TTextFilter< TSharedPtr<FString> >;
	TSharedPtr<FKeyTextFilter> KeyTextFilter;
	TSharedPtr<SSearchBox> KeysSearchBox;

	TSharedPtr<SComboButton> StringTableOptionsCombo;
	TSharedPtr<SListView<TSharedPtr<FAvailableStringTable>>> StringTableOptionsList;
	TSharedPtr<SComboButton> StringTableKeysCombo;
	TSharedPtr<SListView<TSharedPtr<FString>>> StringTableKeysList;

	TArray<TSharedPtr<FAvailableStringTable>> StringTableComboOptions;
	TArray<TSharedPtr<FString>> KeyComboOptions;
};

/** A widget that can be used for editing FText instances */
class STextPropertyEditableTextBox : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(STextPropertyEditableTextBox)
		: _Style(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
		, _Font()
		, _ForegroundColor()
		, _WrapTextAt(0.0f)
		, _AutoWrapText(false)
		, _MinDesiredWidth()
		, _MaxDesiredHeight(300.0f)
		{}
		/** The styling of the textbox */
		SLATE_STYLE_ARGUMENT(FEditableTextBoxStyle, Style)
		/** Font color and opacity (overrides Style) */
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
		/** Text color and opacity (overrides Style) */
		SLATE_ATTRIBUTE(FSlateColor, ForegroundColor)
		/** Whether text wraps onto a new line when it's length exceeds this width; if this value is zero or negative, no wrapping occurs */
		SLATE_ATTRIBUTE(float, WrapTextAt)
		/** Whether to wrap text automatically based on the widget's computed horizontal space */
		SLATE_ATTRIBUTE(bool, AutoWrapText)
		/** When specified, will report the MinDesiredWidth if larger than the content's desired width */
		SLATE_ATTRIBUTE(FOptionalSize, MinDesiredWidth)
		/** When specified, will report the MaxDesiredHeight if smaller than the content's desired height */
		SLATE_ATTRIBUTE(FOptionalSize, MaxDesiredHeight)
	SLATE_END_ARGS()

public:
	UE_API void Construct(const FArguments& Arguments, const TSharedRef<IEditableTextProperty>& InEditableTextProperty);
	UE_API virtual bool SupportsKeyboardFocus() const override;
	UE_API virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;

private:
	UE_API void GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth);
	UE_API bool CanEdit() const;
	UE_API bool IsCultureInvariantFlagEnabled() const;
	UE_API bool IsSourceTextReadOnly() const;
	UE_API bool IsIdentityReadOnly() const;
	UE_API FText GetToolTipText() const;
	UE_API bool IsTextLocalizable() const;

	UE_API FText GetTextValue() const;
	UE_API void OnTextChanged(const FText& NewText);
	UE_API void OnTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);
	UE_API void SetTextError(const FText& InErrorMsg);

	UE_API FText GetNamespaceValue() const;
	UE_API void OnNamespaceChanged(const FText& NewText);
	UE_API void OnNamespaceCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	UE_API FText GetKeyValue() const;
#if USE_STABLE_LOCALIZATION_KEYS
	UE_API void OnKeyChanged(const FText& NewText);
	UE_API void OnKeyCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	UE_API FText GetPackageValue() const;
#endif // USE_STABLE_LOCALIZATION_KEYS

	UE_API FText GetDevNotesValue() const;
	UE_API void OnDevNotesCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	UE_API ECheckBoxState GetLocalizableCheckState() const;

	UE_API void HandleLocalizableCheckStateChanged(ECheckBoxState InCheckboxState);

	UE_API FText GetAdvancedTextSettingsComboToolTip() const;
	UE_API const FSlateBrush* GetAdvancedTextSettingsComboImage() const;

	UE_API bool IsValidIdentity(const FText& InIdentity, FText* OutReason = nullptr, const FText* InErrorCtx = nullptr) const;

	TSharedPtr<IEditableTextProperty> EditableTextProperty;

	TSharedPtr<class SWidget> PrimaryWidget;

	TSharedPtr<SMultiLineEditableTextBox> MultiLineWidget;

	TSharedPtr<SEditableTextBox> SingleLineWidget;

	TSharedPtr<SEditableTextBox> NamespaceEditableTextBox;

	TSharedPtr<SEditableTextBox> KeyEditableTextBox;

	bool bIsMultiLine = false;

	static UE_API FText MultipleValuesText;
};

#undef UE_API
